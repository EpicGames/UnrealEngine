{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Hair Modeling Toolset",
	"Description": "Adds a set of tools for Groom/Hair Modeling to Modeling Tools Mode",
	"Category": "Geometry",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Hidden": false,
	"Modules": [
		{
			"Name": "HairModelingToolset",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64" ]
		},
	],
	"Plugins": [
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolset",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "ModelingToolsEditorMode",
			"Enabled": true
		},
		{
			"Name": "HairStrands",
			"Enabled": true
		}		
	]
}
