// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class HairModelingToolset : ModuleRules
{
	public HairModelingToolset(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Eigen",
                "InteractiveToolsFramework",
				"GeometryCore",
				"GeometryFramework",
				"GeometryAlgorithms",
				"DynamicMesh",
				"MeshConversion",
				"MeshDescription",
                "StaticMeshDescription",
				"ModelingComponents",
				"ModelingOperators",
				"ModelingOperatorsEditorOnly",
				"MeshModelingTools",
				"MeshModelingToolsExp",
				"HairStrandsCore",
				"ModelingToolsEditorMode"

				// ... add other public dependencies that you statically link with here ...
			}
            );
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"InputCore",
				"PhysicsCore",
				"RenderCore",
				"RHI",
				"Niagara",

				"ApplicationCore",
				"Slate",
				"SlateCore",
				"UnrealEd",
				"EditorFramework",
				"ContentBrowser",
				"LevelEditor",
				"StatusBar",
				"Projects"

				// ... add private dependencies that you statically link with here ...
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
