// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenerateLODMeshesTool.h"
#include "InteractiveToolManager.h"
#include "Properties/RemeshProperties.h"
#include "ToolBuilderUtil.h"

#include "ToolSetupUtil.h"
#include "Util/ColorConstants.h"

#include "DynamicMesh/DynamicMesh3.h"

#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"

#include "ModelingObjectsCreationAPI.h"
#include "Selection/ToolSelectionUtil.h"

#include "PrimitiveDrawingUtils.h" // for FPrimitiveDrawInterface

#include "Modules/ModuleManager.h"

#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ModelingToolTargetUtil.h"

#include "IMeshReductionManagerModule.h"
#include "IMeshReductionInterfaces.h"


#if WITH_EDITOR
#include "Misc/ScopedSlowTask.h"
#endif

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGenerateLODMeshesTool"

/*
 * ToolBuilder
 */
USingleSelectionMeshEditingTool* UGenerateLODMeshesToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UGenerateLODMeshesTool>(SceneState.ToolManager);
}

/*
 * Tool
 */
UGenerateLODMeshesToolProperties::UGenerateLODMeshesToolProperties()
{
	GroupBoundaryConstraint = EGroupBoundaryConstraint::Ignore;
	MaterialBoundaryConstraint = EMaterialBoundaryConstraint::Ignore;

	// hardcoded for hair helmet

	FLODLevelGenerateSettings LOD0;
	LOD0.SimplifierType = ESimplifyType::UEStandard;
	LOD0.TargetMode = ESimplifyTargetType::VertexCount;
	LOD0.TargetPercentage = 50;
	LOD0.TargetCount = 500;
	LODLevels.Add(LOD0);

	FLODLevelGenerateSettings LOD1;
	LOD1.SimplifierType = ESimplifyType::UEStandard;
	LOD1.TargetMode = ESimplifyTargetType::VertexCount;
	LOD1.TargetPercentage = 30;
	LOD1.TargetCount = 250;
	LODLevels.Add(LOD1);

	FLODLevelGenerateSettings LOD2;
	LOD2.SimplifierType = ESimplifyType::UEStandard;
	LOD2.TargetMode = ESimplifyTargetType::VertexCount;
	LOD2.TargetPercentage = 15;
	LOD2.TargetCount = 150;
	LODLevels.Add(LOD2);
}

void UGenerateLODMeshesTool::Setup()
{
	UInteractiveTool::Setup();

	SetToolDisplayName(LOCTEXT("ToolName", "Generate HairHelmet LODs"));

	{
		// if in editor, create progress indicator dialog because building mesh copies can be slow (for very large meshes)
		// this is especially needed because of the copy we make of the meshdescription; for Reasons, copying meshdescription is pretty slow
#if WITH_EDITOR
		static const FText SlowTaskText = LOCTEXT("GenerateLODMeshesInit", "Building mesh simplification data...");

		FScopedSlowTask SlowTask(3.0f, SlowTaskText);
		SlowTask.MakeDialog();

		// Declare progress shortcut lambdas
		auto EnterProgressFrame = [&SlowTask](int Progress)
	{
			SlowTask.EnterProgressFrame((float)Progress);
		};
#else
		auto EnterProgressFrame = [](int Progress) {};
#endif
		const FMeshDescription* TargetMeshDescription = UE::ToolTarget::GetMeshDescription(Target);
		OriginalMeshDescription = MakeShared<FMeshDescription, ESPMode::ThreadSafe>(*TargetMeshDescription);
		EnterProgressFrame(1);
		// aux-data isn't deep copied - by default it is built during initial evaluation (not thread safe)  
		// so force aux-data rebuild now before multiple UE4 simplifiers try to use it in parallel.
		OriginalMeshDescription->BuildIndexers();   
		OriginalMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
		FMeshDescriptionToDynamicMesh Converter;
		Converter.Convert(TargetMeshDescription, *OriginalMesh);
		EnterProgressFrame(2);
		OriginalMeshSpatial = MakeShared<FDynamicMeshAABBTree3, ESPMode::ThreadSafe>(OriginalMesh.Get(), true);
	}

	WorldBounds = FAxisAlignedBox3d(Cast<IPrimitiveComponentBackedTarget>(Target)->GetOwnerActor()->GetComponentsBoundingBox());

	// initialize our properties
	SimplifyProperties = NewObject<UGenerateLODMeshesToolProperties>(this);
	SimplifyProperties->RestoreProperties(this);
	AddToolPropertySource(SimplifyProperties);

	SimplifyProperties->WatchProperty(SimplifyProperties->TargetMode, [this](ESimplifyTargetType) { InvalidateAllPreviews(); });
	SimplifyProperties->WatchProperty(SimplifyProperties->SimplifierType, [this](ESimplifyType) { InvalidateAllPreviews(); });
	SimplifyProperties->WatchProperty(SimplifyProperties->TargetPercentage, [this](int32) { InvalidateAllPreviews(); });
	SimplifyProperties->WatchProperty(SimplifyProperties->TargetEdgeLength, [this](int32) { InvalidateAllPreviews(); });
	SimplifyProperties->WatchProperty(SimplifyProperties->TargetCount, [this](int32) { InvalidateAllPreviews(); });
	SimplifyProperties->WatchProperty(SimplifyProperties->bDiscardAttributes, [this](bool) { InvalidateAllPreviews(); });
	SimplifyProperties->WatchProperty(SimplifyProperties->bReproject, [this](bool) { InvalidateAllPreviews(); });

	SimplifyProperties->WatchProperty(SimplifyProperties->bShowGroupColors,
									  [this](bool bNewValue) { UpdateVisualization(); });
	SimplifyProperties->WatchProperty(SimplifyProperties->bShowWireframe,
									  [this](bool bNewValue) { UpdateVisualization(); });

	UpdateNumPreviews();

	UpdateVisualization();
}


void UGenerateLODMeshesTool::OnShutdown(EToolShutdownType ShutdownType)
{
	SimplifyProperties->SaveProperties(this);
	Cast<IPrimitiveComponentBackedTarget>(Target)->SetOwnerVisibility(true);

	if (ShutdownType == EToolShutdownType::Accept)
	{
		GenerateAssets();
	}
	else
	{
		for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
		{
			Preview->Shutdown();
		}
	}
}


void UGenerateLODMeshesTool::OnTick(float DeltaTime)
{
	// invalidate any previews that have invalid caches
	for (int32 k = 0; k < CachedLODLevels.Num() && k < SimplifyProperties->LODLevels.Num(); ++k)
	{
		if (CachedLODLevels[k] != SimplifyProperties->LODLevels[k])
		{
			CachedLODLevels[k] = SimplifyProperties->LODLevels[k];
			PreviewFactories[k]->LODSettings = SimplifyProperties->LODLevels[k];
			Previews[k]->InvalidateResult();
		}
	}


	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Preview->Tick(DeltaTime);
	}
}


void UGenerateLODMeshesTool::UpdateNumPreviews()
{
	int32 NumPreviews = SimplifyProperties->LODLevels.Num();
	int32 CurNumPreviews = Previews.Num();

	check(CurNumPreviews < NumPreviews);		// todo: support less

	FTransform OrigTransform = Cast<IPrimitiveComponentBackedTarget>(Target)->GetWorldTransform();
	FVector WorldShift = (FVector)((WorldBounds.Width() * 1.1) * FVector3d::UnitX());

	for (int32 k = CurNumPreviews; k < NumPreviews; ++k)
	{
		TUniquePtr<FGenerateLODOperatorFactory> Factory = MakeUnique<FGenerateLODOperatorFactory>();
		Factory->ParentTool = this;

		UMeshOpPreviewWithBackgroundCompute* NewPreview = NewObject<UMeshOpPreviewWithBackgroundCompute>(this);
		NewPreview->Setup(GetTargetWorld(), Factory.Get());

		FComponentMaterialSet MaterialSet;
		Cast<IMaterialProvider>(Target)->GetMaterialSet(MaterialSet);
		NewPreview->ConfigureMaterials(MaterialSet.Materials,
			ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager())
		);

		NewPreview->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
		NewPreview->PreviewMesh->UpdatePreview(OriginalMesh.Get());

		FTransform UseTransform = OrigTransform;
		UseTransform.AddToTranslation( (float)(k+1)*WorldShift );
		NewPreview->PreviewMesh->SetTransform(UseTransform);

		Factory->UseTransform = UseTransform;

		Previews.Add(NewPreview);
		PreviewFactories.Add(MoveTemp(Factory)); 

		NewPreview->OnMeshUpdated.AddLambda([this](UMeshOpPreviewWithBackgroundCompute* Compute)
		{
			OnPreviewUpdated(Compute);
		});
	}

	for (int32 k = 0; k < Previews.Num(); ++k)
	{
		PreviewFactories[k]->LODSettings = SimplifyProperties->LODLevels[k];
	}

	CachedLODLevels.Reset();
	CachedLODLevels.SetNum(Previews.Num());
		
	InvalidateAllPreviews();
}


void UGenerateLODMeshesTool::InvalidateAllPreviews()
{
	// just invalidate caches, next Tick() will kick off new computes
	for (int32 k = 0; k < Previews.Num(); ++k)
	{
		CachedLODLevels[k] = FLODLevelGenerateSettings();
	}
}


void UGenerateLODMeshesTool::OnPreviewUpdated(UMeshOpPreviewWithBackgroundCompute* PreviewCompute)
{
	for (int32 k = 0; k < Previews.Num(); ++k)
	{
		if (Previews[k] == PreviewCompute)
		{
			const FDynamicMesh3* Mesh = Previews[k]->PreviewMesh->GetMesh();
			const FDynamicMeshUVOverlay* UVLayer = Mesh->Attributes()->GetUVLayer(0);

			FLODLevelGenerateSettings& GenerateSettings = SimplifyProperties->LODLevels[k];
			GenerateSettings.Result = FString::Printf(TEXT("V:%d  T:%d  U:%d"), 
				Mesh->VertexCount(), Mesh->TriangleCount(), UVLayer->ElementCount());

			return;
		}
	}
}


TUniquePtr<UE::Geometry::FDynamicMeshOperator> FGenerateLODOperatorFactory::MakeNewOperator()
{
	TUniquePtr<UE::Geometry::FSimplifyMeshOp> Op = MakeUnique<UE::Geometry::FSimplifyMeshOp>();

	UGenerateLODMeshesToolProperties* SimplifyProperties = ParentTool->SimplifyProperties;

	Op->SimplifierType = LODSettings.SimplifierType;
	Op->TargetMode = LODSettings.TargetMode;
	Op->TargetCount = LODSettings.TargetCount;
	Op->TargetPercentage = LODSettings.TargetPercentage;
	Op->bReproject = LODSettings.bReproject;

	Op->TargetEdgeLength = SimplifyProperties->TargetEdgeLength;
	Op->bDiscardAttributes = SimplifyProperties->bDiscardAttributes;
	Op->bPreventNormalFlips = SimplifyProperties->bPreventNormalFlips;
	Op->bPreserveSharpEdges = SimplifyProperties->bPreserveSharpEdges;
	Op->bAllowSeamCollapse = !SimplifyProperties->bPreserveSharpEdges;
	Op->MeshBoundaryConstraint = (EEdgeRefineFlags)SimplifyProperties->MeshBoundaryConstraint;
	Op->GroupBoundaryConstraint = (EEdgeRefineFlags)SimplifyProperties->GroupBoundaryConstraint;
	Op->MaterialBoundaryConstraint = (EEdgeRefineFlags)SimplifyProperties->MaterialBoundaryConstraint;
	FTransform LocalToWorld = this->UseTransform;
	Op->SetTransform(LocalToWorld);

	Op->OriginalMeshDescription = ParentTool->OriginalMeshDescription;
	Op->OriginalMesh = ParentTool->OriginalMesh;
	Op->OriginalMeshSpatial = ParentTool->OriginalMeshSpatial;

	IMeshReductionManagerModule& MeshReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface");
	Op->MeshReduction = MeshReductionModule.GetStaticMeshReductionInterface();

	return Op;
}



void UGenerateLODMeshesTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	//FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();
	//FTransform Transform = Cast<IPrimitiveComponentBackedTarget>(Target)->GetWorldTransform(); //Actor->GetTransform();

	//FColor LineColor(255, 0, 0);
	//const FDynamicMesh3* TargetMesh = Preview->PreviewMesh->GetPreviewDynamicMesh();
	//if (TargetMesh->HasAttributes())
	//{
	//	float PDIScale = RenderAPI->GetCameraState().GetPDIScalingFactor();

	//	const FDynamicMeshUVOverlay* UVOverlay = TargetMesh->Attributes()->PrimaryUV();
	//	for (int eid : TargetMesh->EdgeIndicesItr())
	//	{
	//		if (UVOverlay->IsSeamEdge(eid))
	//		{
	//			FVector3d A, B;
	//			TargetMesh->GetEdgeV(eid, A, B);
	//			PDI->DrawLine(Transform.TransformPosition((FVector)A), Transform.TransformPosition((FVector)B),
	//				LineColor, 0, 2.0*PDIScale, 1.0f, true);
	//		}
	//	}
	//}
}



void UGenerateLODMeshesTool::UpdateVisualization()
{
	IMaterialProvider* TargetMaterial = Cast<IMaterialProvider>(Target);
	FComponentMaterialSet MaterialSet;
	TargetMaterial->GetMaterialSet(MaterialSet);

	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Preview->PreviewMesh->EnableWireframe(SimplifyProperties->bShowWireframe);

		Preview->ConfigureMaterials(MaterialSet.Materials,
			ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()));
	}

	//FComponentMaterialSet MaterialSet;
	//if (SimplifyProperties->bShowGroupColors)
	//{
	//	MaterialSet.Materials = {ToolSetupUtil::GetSelectionMaterial(GetToolManager())};
	//	Preview->PreviewMesh->SetTriangleColorFunction([this](const FDynamicMesh3* Mesh, int TriangleID)
	//	{
	//		return LinearColors::SelectFColor(Mesh->GetTriangleGroup(TriangleID));
	//	},
	//	UPreviewMesh::ERenderUpdateMode::FastUpdate);
	//}
	//else
	//{
	//	TargetMaterial->GetMaterialSet(MaterialSet);
	//	Preview->PreviewMesh->ClearTriangleColorFunction(UPreviewMesh::ERenderUpdateMode::FastUpdate);
	//}



}

bool UGenerateLODMeshesTool::CanAccept() const
{
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		if (!Preview->HaveValidResult())
		{
			return false;
		}
	}
	return Super::CanAccept();
}

void UGenerateLODMeshesTool::GenerateAssets()
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("GenerateLODMeshesToolTransactionName", "Simplify Mesh"));

	IPrimitiveComponentBackedTarget* TargetComponent = Cast<IPrimitiveComponentBackedTarget>(Target);
	FTransform Transform = TargetComponent->GetWorldTransform();

	int32 NumLODs = Previews.Num();
	for (int32 k = 0; k < NumLODs; ++k)
	{
		UMeshOpPreviewWithBackgroundCompute* Preview = Previews[k];

		FDynamicMeshOpResult OpResult = Preview->Shutdown();
		if (OpResult.Mesh->TriangleCount() == 0)
		{
			continue;		// failed!
		}

		FComponentMaterialSet MaterialSet;
		Cast<IMaterialProvider>(Target)->GetMaterialSet(MaterialSet);

		FString BaseName = UE::Modeling::GetComponentAssetBaseName(TargetComponent->GetOwnerComponent());
		FString Name = FString::Printf( TEXT("%s_LOD%d"), *BaseName, (SimplifyProperties->NameIndexBase+k) );

		FCreateMeshObjectParams NewMeshObjectParams;
		NewMeshObjectParams.TargetWorld = GetTargetWorld();
		NewMeshObjectParams.Transform = Transform;
		NewMeshObjectParams.BaseName = Name;
		NewMeshObjectParams.Materials = MaterialSet.Materials;
		NewMeshObjectParams.SetMesh(OpResult.Mesh.Get());
		FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
		if (Result.IsOK() && Result.NewActor != nullptr)
		{
			ToolSelectionUtil::SetNewActorSelection(GetToolManager(), Result.NewActor);
		}
	}

	GetToolManager()->EndUndoTransaction();
}




#undef LOCTEXT_NAMESPACE
