// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomCardsEditorTool.h"
#include "InteractiveToolManager.h"
#include "PrimitiveDrawingUtils.h"
#include "ToolBuilderUtil.h"

#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMeshEditor.h"
#include "Util/ColorConstants.h"
#include "ToolSetupUtil.h"
#include "Selection/ToolSelectionUtil.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMeshEditor.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "Parameterization/MeshUVTransforms.h"
#include "ModelingToolTargetUtil.h"
#include "DynamicMeshToMeshDescription.h"

#include "Changes/MeshVertexChange.h"

#include "GroomQueryUtil.h"

#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"


#define LOCTEXT_NAMESPACE "UGroomCardsEditorTool"

using namespace UE::Geometry;

/*
 * ToolBuilder
 */

UMeshSurfacePointTool* UGroomCardsEditorToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UGroomCardsEditorTool* NewTool = NewObject<UGroomCardsEditorTool>(SceneState.ToolManager);
	NewTool->SetWorld(SceneState.World);
	return NewTool;
}


void UEditGroomCardsToolActionPropertySet::PostAction(EEditGroomCardsToolActions Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(Action);
	}
}



/*
 * Tool
 */
UGroomCardsEditorTool::UGroomCardsEditorTool()
{
	SetToolDisplayName(LOCTEXT("GroomCardsEditorToolName", "Groom to Mesh"));
}




void UGroomCardsEditorTool::Setup()
{
	UMeshSurfacePointTool::Setup();

	PreviewMesh = NewObject<UPreviewMesh>(this);
	PreviewMesh->bBuildSpatialDataStructure = true;
	PreviewMesh->CreateInWorld(TargetWorld, FTransform::Identity);
	PreviewMesh->SetTransform((FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target));

	MeshMaterial = ToolSetupUtil::GetDefaultSculptMaterial(GetToolManager());
	UVMaterial = ToolSetupUtil::GetUVCheckerboardMaterial(50.0);
	PreviewMesh->SetMaterial(MeshMaterial);

	// Set up SelectionMechanic but remove its behaviors because we'd rather deal with clicks
	// ourselves. Specifically, we want to be able to start new transactions on clicks, which
	// we currently shouldn't do in an OnSelectionModifiedEvent call.
	CardMeshSelectionMechanic = NewObject<UPolygonSelectionMechanic>(this);
	CardMeshSelectionMechanic->bAddSelectionFilterPropertiesToParentTool = false;
	CardMeshSelectionMechanic->Setup(this);
	CardMeshSelectionMechanic->OnSelectionChanged.AddUObject(this, &UGroomCardsEditorTool::OnSelectionModifiedEvent);
	CardMeshSelectionMechanic->PolyEdgesRenderer.LineThickness = 1.0;
	CardMeshSelectionMechanic->DisableBehaviors(this);

	// configure secondary render material
	UMaterialInterface* SelectionMaterial = ToolSetupUtil::GetSelectionMaterial(FLinearColor(0.9f, 0.1f, 0.1f), GetToolManager());
	if (SelectionMaterial != nullptr)
	{
		PreviewMesh->SetSecondaryRenderMaterial(SelectionMaterial);
	}

	// enable secondary triangle buffers
	PreviewMesh->EnableSecondaryTriangleBuffers(
		[this](const FDynamicMesh3* Mesh, int32 TriangleID)
	{
		return CardMeshSelectionMechanic->GetActiveSelection().IsSelectedTriangle(Mesh, Topology.Get(), TriangleID);
	});


	PreviewGeom = NewObject<UPreviewGeometry>(this);
	PreviewGeom->CreateInWorld(TargetWorld, FTransform::Identity);
	PreviewGeom->GetActor()->SetActorTransform((FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target));

	UE::ToolTarget::HideSourceObject(Target);

	ControlPointsMechanic = NewObject<USpaceCurveDeformationMechanic>(this);
	ControlPointsMechanic->Setup(this);
	ControlPointsMechanic->SetWorld(TargetWorld);
	ControlPointsMechanic->OnPointsChanged.AddLambda([this]() 
	{
		bCurveUpdatePending = true;
	});


	// register click behavior
	USingleClickInputBehavior* ClickBehavior = NewObject<USingleClickInputBehavior>();
	ClickBehavior->Initialize(this);
	AddInputBehavior(ClickBehavior);

	// configure behavior priorities
	ClickBehavior->SetDefaultPriority(ControlPointsMechanic->ClickBehavior->GetPriority().MakeLower());
	// hardcode these higher so they override the behaviors in UMeshSurfacePointTool that are not exposed
	ControlPointsMechanic->ClickBehavior->SetDefaultPriority(FInputCapturePriority(50));
	ControlPointsMechanic->HoverBehavior->SetDefaultPriority(FInputCapturePriority(50));


	InfoProperties = NewObject<UGroomCardsInfoToolProperties>(this);
	InfoProperties->RestoreProperties(this);
	AddToolPropertySource(InfoProperties);

	ControlPointsMechanic->TransformProperties->RestoreProperties(this);
	AddToolPropertySource(ControlPointsMechanic->TransformProperties);

	SelectActions = NewObject<USelectGroomCardsToolActions>();
	SelectActions->Initialize(this);
	AddToolPropertySource(SelectActions);

	EditActions = NewObject<UEditGroomCardsToolActions>();
	EditActions->Initialize(this);
	AddToolPropertySource(EditActions);

	bSetupValid = false;
	bVisualizationChanged = true;
	PendingAction = EEditGroomCardsToolActions::NoAction;

	InitializeMesh();
	bSetupValid = true;

	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Edit Groom Cards"),
		EToolMessageLevel::UserNotification);

	OnPreviewMeshChangedHandle = PreviewMesh->GetOnMeshChanged().Add(
		FSimpleMulticastDelegate::FDelegate::CreateUObject(this, &UGroomCardsEditorTool::OnPreviewMeshChanged));
}


void UGroomCardsEditorTool::Shutdown(EToolShutdownType ShutdownType)
{
	UMeshSurfacePointTool::Shutdown(ShutdownType);

	CardMeshSelectionMechanic->Shutdown();

	ControlPointsMechanic->TransformProperties->SaveProperties(this);
	ControlPointsMechanic->Shutdown();

	PreviewMesh->GetOnMeshChanged().Remove(OnPreviewMeshChangedHandle);

	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("CardsMeshApplyEditChange", "Update Cards Mesh"));
		PreviewMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			UE::ToolTarget::CommitDynamicMeshUpdate(Target, ReadMesh, false);
		});
		GetToolManager()->EndUndoTransaction();
	}

	PreviewMesh->Disconnect();
	PreviewMesh = nullptr;

	PreviewGeom->Disconnect();
	PreviewGeom = nullptr;

	UE::ToolTarget::ShowSourceObject(Target);
}


void UGroomCardsEditorTool::OnTick(float DeltaTime)
{
	if (PendingAction != EEditGroomCardsToolActions::NoAction)
	{
		if (PendingAction == EEditGroomCardsToolActions::Delete)
		{
			ApplyDeleteAction();
		}
		else if (PendingAction == EEditGroomCardsToolActions::SelectionClear)
		{
			if (ControlPointsMechanic)
			{
				ControlPointsMechanic->SelectionClear();
			}
		}
		else if (PendingAction == EEditGroomCardsToolActions::SelectionFill)
		{
			if (ControlPointsMechanic)
			{
				ControlPointsMechanic->SelectionFill();
			}
		}
		else if (PendingAction == EEditGroomCardsToolActions::SelectionAddNext)
		{
			if (ControlPointsMechanic)
			{
				ControlPointsMechanic->SelectionGrowToNext();
			}
		}
		else if (PendingAction == EEditGroomCardsToolActions::SelectionAddPrevious)
		{
			if (ControlPointsMechanic)
			{
				ControlPointsMechanic->SelectionGrowToPrev();
			}
		}
		else if (PendingAction == EEditGroomCardsToolActions::SelectionAddToEnd)
		{
			if (ControlPointsMechanic)
			{
				ControlPointsMechanic->SelectionGrowToEnd();
			}
		}
		else if (PendingAction == EEditGroomCardsToolActions::SelectionAddToStart)
		{
			if (ControlPointsMechanic)
			{
				ControlPointsMechanic->SelectionGrowToStart();
			}
		}
		PendingAction = EEditGroomCardsToolActions::NoAction;
	}

	if (bVisualizationChanged)
	{
		//PreviewMesh->SetMaterial(Settings->bShowUVs ? UVMaterial : MeshMaterial);
		bVisualizationChanged = false;
	}


	if (bSelectionStateDirty)
	{
		UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(PreviewMesh->GetRootComponent());		// ugh
		if (DynamicMeshComponent)
		{
			DynamicMeshComponent->FastNotifySecondaryTrianglesChanged();
		}
		bSelectionStateDirty = false;
	}

	if (bCurveUpdatePending)
	{
		UpdateOnCurveEdit();
		bCurveUpdatePending = false;
	}

	if (ControlPointsMechanic)
	{
		ControlPointsMechanic->Tick(DeltaTime);
	}
}




bool UGroomCardsEditorTool::HasAccept() const
{
	return true;
}

bool UGroomCardsEditorTool::CanAccept() const
{
	return true;
}







bool UGroomCardsEditorTool::HitTest(const FRay& WorldRay, FHitResult& OutHit)
{
	return CardMeshSelectionMechanic->TopologyHitTest(WorldRay, OutHit);
}



FInputRayHit UGroomCardsEditorTool::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	FHitResult OutHit;
	if (HitTest(ClickPos.WorldRay, OutHit))
	{
		return FInputRayHit(OutHit.Distance);
	}

	// background capture, if nothing else is hit
	return FInputRayHit(TNumericLimits<float>::Max());
}







FInputRayHit UGroomCardsEditorTool::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	return FInputRayHit();
}

void UGroomCardsEditorTool::OnBeginDrag(const FRay& WorldRay)
{
	check(false);
}

void UGroomCardsEditorTool::OnUpdateDrag(const FRay& Ray)
{
	check(false);
}

void UGroomCardsEditorTool::OnEndDrag(const FRay& Ray)
{
	check(false);
}

void UGroomCardsEditorTool::OnCancelDrag()
{
	check(false);
}


bool UGroomCardsEditorTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	//if (ActiveVertexChange == nullptr && MultiTransformer->InGizmoEdit() == false)
	//{
	CardMeshSelectionMechanic->UpdateHighlight(DevicePos.WorldRay);
	//}
	return true;
}

void UGroomCardsEditorTool::OnEndHover()
{
	CardMeshSelectionMechanic->ClearHighlight();
}






class FEditableGroomCardSet
{
public:

	TSharedPtr<FDynamicMesh3> FullCardMesh;

	TSharedPtr<UE::GroomQueries::FMeshCardStripSet> CardStrips;


	void Initialize(const FMeshDescription* MeshDescripIn)
	{
		FullCardMesh = MakeShared<FDynamicMesh3>();
		FullCardMesh->EnableTriangleGroups(0);
		FullCardMesh->EnableAttributes();

		CardStrips = MakeShared<UE::GroomQueries::FMeshCardStripSet>();

		UE::GroomQueries::ExtractAllHairCards(MeshDescripIn, *FullCardMesh, *CardStrips);
		UE::GroomQueries::ExtractCardQuads(*FullCardMesh, *CardStrips);
		UE::GroomQueries::ExtractCardCurves(*FullCardMesh, *CardStrips);
	}
};



struct FCardCurvePointInfo
{
	FIndex2i Vertices;
	FVector3d LocalPoints[2];
};

class FGroomCardEdit
{
public:
	int32 CardGroupID;

	UE::GroomQueries::FMeshCardStrip InitialCard;

	UE::GroomQueries::FMeshCardStrip EditedCard;

	TArray<FFrame3d> InitialFrames;
	TArray<FFrame3d> EditedFrames;

	TArray<FCardCurvePointInfo> CurveMeshVertices;

	void InitializeEncoding(const FDynamicMesh3* Mesh)
	{
		int32 NumCurveVerts = InitialCard.CardCurve.Num();
		CurveMeshVertices.SetNum(NumCurveVerts);
		for ( int32 k = 0; k < NumCurveVerts; ++k )
		{
			FCardCurvePointInfo VertInfo;
			VertInfo.Vertices = InitialCard.GetCurvePointVertices(k);
			VertInfo.LocalPoints[0] = InitialFrames[k].ToFramePoint(Mesh->GetVertex(VertInfo.Vertices.A));
			VertInfo.LocalPoints[1] = InitialFrames[k].ToFramePoint(Mesh->GetVertex(VertInfo.Vertices.B));
			CurveMeshVertices[k] = VertInfo;
		}
	}

	void UpdateFromEditedFrames(FDynamicMesh3& Mesh, FMeshVertexChangeBuilder* Tracker)
	{
		int32 NumCurveVerts = InitialCard.CardCurve.Num();
		for (int32 k = 0; k < NumCurveVerts; ++k)
		{
			for (int32 j = 0; j < 2; ++j)
			{
				int32 VertIndex = CurveMeshVertices[k].Vertices[j];
				FVector3d LocalPoint = CurveMeshVertices[k].LocalPoints[j];
				FVector3d MeshPoint = EditedFrames[k].FromFramePoint(LocalPoint);
				if (Tracker)
				{
					Tracker->SaveVertexInitial(&Mesh, VertIndex);
				}
				Mesh.SetVertex(VertIndex, MeshPoint);
			}

			EditedCard.CardCurve[k] = EditedFrames[k].Origin;
			EditedCard.CardCurveFrames[k] = EditedFrames[k];
		}

		// todo: normals?
	}
};




class FGroomCardEditChange : public FToolCommandChange
{
public:
	FGroomCardEdit Edit;

	virtual void Apply(UObject* Object) override
	{
		Cast<UGroomCardsEditorTool>(Object)->ApplyGroomCardEdit(Edit, false);
	}

	virtual void Revert(UObject* Object) override
	{
		Cast<UGroomCardsEditorTool>(Object)->ApplyGroomCardEdit(Edit, true);
	}
};






void UGroomCardsEditorTool::OnClicked(const FInputDeviceRay& ClickPos)
{
	// update selection
	GetToolManager()->BeginUndoTransaction(LOCTEXT("CardsEditSelectionChange", "Selection"));

	CardMeshSelectionMechanic->BeginChange();
	FVector3d LocalHitPosition, LocalHitNormal;
	bool bSelectionModified = CardMeshSelectionMechanic->UpdateSelection(ClickPos.WorldRay, LocalHitPosition, LocalHitNormal);
	CardMeshSelectionMechanic->EndChangeAndEmitIfModified();

	if (bSelectionModified)
	{
		// close any pending card edit
		CompleteActiveCardEditAndEmitChanges();

		// if we have a new single selection, start a new edit
		const FGroupTopologySelection& CurSelection = CardMeshSelectionMechanic->GetActiveSelection();
		if (CurSelection.SelectedGroupIDs.Num() == 1)
		{
			if (const int32* SelectedCardGroupID = CurSelection.SelectedGroupIDs.FindArbitraryElement())
			{
				GetToolManager()->EmitObjectChange(this, MakeUnique<FBeginGroomCardsEditChange>(*SelectedCardGroupID),
					LOCTEXT("BeginCardEditChange", "BeginCardEdit"));

				BeginCardEdit(*SelectedCardGroupID);
			}
		}

	}

	GetToolManager()->EndUndoTransaction();
}




void UGroomCardsEditorTool::CompleteActiveCardEditAndEmitChanges()
{
	// if we have an active edit, complete it
	if (ActiveCardEdit)
	{
		// save current selection from control points mechanic first, so that when we undo, 
		// control points will be initialized before we restore selection
		ControlPointsMechanic->ClearSelection();

		TUniquePtr<FEndGroomCardsEditChange> EndCardEditChange = MakeUnique<FEndGroomCardsEditChange>();
		EndCardEditChange->CardEdit = MakePimpl<FGroomCardEdit>();
		*(EndCardEditChange->CardEdit) = *ActiveCardEdit;
		GetToolManager()->EmitObjectChange(this, MoveTemp(EndCardEditChange), LOCTEXT("EndCardEditChange", "EndCardEdit"));

		// save change to mesh and card set if they were modified
		if (bActiveCardEditUpdated)
		{
			//EndMoveChange();
			PreviewMesh->ForceRebuildSpatial();

			TUniquePtr<FGroomCardEditChange> CardSetUpdateChange = MakeUnique<FGroomCardEditChange>();
			CardSetUpdateChange->Edit = *ActiveCardEdit;
			CardSetUpdateChange->Apply(this);
			GetToolManager()->EmitObjectChange(this, MoveTemp(CardSetUpdateChange), LOCTEXT("CardSetUpdateChange", "UpdateCardSet"));
		}

		EndCardEdit();
	}
}




void UGroomCardsEditorTool::BeginCardEdit(int32 CardGroupID)
{
	const UE::GroomQueries::FMeshCardStrip& CardStrip = EditableCardSet->CardStrips->FindStripForGroup(CardGroupID);

	FTransformSRT3d Transform(PreviewMesh->GetTransform());
	check(FMathd::Abs(Transform.TransformVector(UE::Geometry::Normalized(FVector3d(1,1,1))).SquaredLength() - 1.0) < FMathd::ZeroTolerance);
	FFrame3d TransformFrame(Transform.GetTranslation(), Transform.GetRotation());

	TArray<FFrame3d> LocalFrames;
	for (FFrame3d LocalFrame : CardStrip.CardCurveFrames)
	{
		LocalFrames.Add(LocalFrame);
	}

	// initialize current edit
	ActiveCardEdit = MakePimpl<FGroomCardEdit>();
	ActiveCardEdit->CardGroupID = CardGroupID;
	ActiveCardEdit->InitialCard = CardStrip;
	ActiveCardEdit->EditedCard = ActiveCardEdit->InitialCard;
	ActiveCardEdit->InitialFrames = LocalFrames;
	ActiveCardEdit->EditedFrames = ActiveCardEdit->InitialFrames;
	ActiveCardEdit->InitializeEncoding(PreviewMesh->GetMesh());

	// initialize new curve adapter
	CurveSourceAdapter = MakeShared<FSpaceCurveSource>();
	CurveSourceAdapter->GetPointCount = [this]() { check(ActiveCardEdit.IsValid());  return ActiveCardEdit->EditedFrames.Num(); };
	CurveSourceAdapter->GetPoint = [this, TransformFrame](int32 Index) { return TransformFrame.FromFrame(ActiveCardEdit->EditedFrames[Index]); };
	CurveSourceAdapter->IsLoop = []() { return false; };
	ControlPointsMechanic->SetCurveSource(CurveSourceAdapter);

	ActiveCardGroupID = CardGroupID;
	bActiveCardEditUpdated = false;
}



void UGroomCardsEditorTool::RestoreCardEdit(const FGroomCardEdit* RestoreEdit)
{
	// this should probably be cached once...
	FTransformSRT3d Transform(PreviewMesh->GetTransform());
	check(FMathd::Abs(Transform.TransformVector(UE::Geometry::Normalized(FVector3d(1, 1, 1))).SquaredLength() - 1.0) < FMathd::ZeroTolerance);
	FFrame3d TransformFrame(Transform.GetTranslation(), Transform.GetRotation());

	ActiveCardEdit = MakePimpl<FGroomCardEdit>();
	*ActiveCardEdit = *RestoreEdit;

	// initialize new curve adapter
	CurveSourceAdapter = MakeShared<FSpaceCurveSource>();
	CurveSourceAdapter->GetPointCount = [this]() { check(ActiveCardEdit.IsValid());  return ActiveCardEdit->EditedFrames.Num(); };
	CurveSourceAdapter->GetPoint = [this, TransformFrame](int32 Index) { return TransformFrame.FromFrame(ActiveCardEdit->EditedFrames[Index]); };
	CurveSourceAdapter->IsLoop = []() { return false; };
	ControlPointsMechanic->SetCurveSource(CurveSourceAdapter);

	// update mesh with this edit result
	//UpdateOnCurveEdit();
	bCurveUpdatePending = true;

	ActiveCardGroupID = ActiveCardEdit->CardGroupID;
	bActiveCardEditUpdated = false;
}




void UGroomCardsEditorTool::EndCardEdit()
{
	ActiveCardEdit = nullptr;
	ControlPointsMechanic->ClearCurveSource();
	CurveSourceAdapter = nullptr;
	ActiveCardGroupID = -1;
	bActiveCardEditUpdated = false;
}





void UGroomCardsEditorTool::UpdateOnCurveEdit()
{
	check(ActiveCardEdit && ActiveCardEdit.IsValid());

	if (bActiveCardEditUpdated == false)
	{
		bActiveCardEditUpdated = true;
		//BeginMoveChange();
	}

	FTransformSRT3d Transform(PreviewMesh->GetTransform());
	check( FMathd::Abs(Transform.TransformVector(UE::Geometry::Normalized(FVector3d(1,1,1))).SquaredLength() - 1.0) < FMathd::ZeroTolerance );
	FFrame3d TransformFrame(Transform.GetTranslation(), Transform.GetRotation());

	TArray<FFrame3d> NewPositions;
	ControlPointsMechanic->GetCurrentCurvePoints(NewPositions);

	for (int32 k = 0; k < ActiveCardEdit->EditedFrames.Num(); ++k)
	{
		FFrame3d LocalFrame = TransformFrame.ToFrame(NewPositions[k]);
		ActiveCardEdit->EditedFrames[k] = LocalFrame;
	}

	PreviewMesh->DeferredEditMesh([&](FDynamicMesh3& Mesh)
	{
		//ActiveCardEdit->UpdateFromEditedFrames(Mesh, ActiveVertexChange.Get());
		ActiveCardEdit->UpdateFromEditedFrames(Mesh, nullptr);
	}, false);

	PreviewMesh->NotifyDeferredEditCompleted(UPreviewMesh::ERenderUpdateMode::FastUpdate, 
		EMeshRenderAttributeFlags::Positions, false);
}



void UGroomCardsEditorTool::ApplyGroomCardEdit(const FGroomCardEdit& Edit, bool bIsRevert)
{
	UE::GroomQueries::FMeshCardStrip& CardStrip = EditableCardSet->CardStrips->FindStripForGroup(Edit.CardGroupID);

	const UE::GroomQueries::FMeshCardStrip& SetToStrip = (bIsRevert) ? Edit.InitialCard : Edit.EditedCard;

	CardStrip.CardCurve = SetToStrip.CardCurve;
	CardStrip.CardCurveFrames = SetToStrip.CardCurveFrames;
}




void UGroomCardsEditorTool::OnSelectionModifiedEvent()
{
	bSelectionStateDirty = true;
}





void UGroomCardsEditorTool::BeginMoveChange()
{
	check(ActiveVertexChange == nullptr);
	ActiveVertexChange = new FMeshVertexChangeBuilder(EMeshVertexChangeComponents::VertexPositions | EMeshVertexChangeComponents::OverlayNormals);
}

void UGroomCardsEditorTool::EndMoveChange()
{
	check(ActiveVertexChange);
	if (ActiveVertexChange->SavedVertices.Num() > 0)
	{

		TUniquePtr<TWrappedToolCommandChange<FMeshVertexChange>> NewChange = MakeUnique<TWrappedToolCommandChange<FMeshVertexChange>>();
		NewChange->WrappedChange = MoveTemp(ActiveVertexChange->Change);
		//NewChange->BeforeModify = [this](bool bRevert)
		//{
		//	this->WaitForPendingUndoRedo();
		//};

		GetToolManager()->EmitObjectChange(PreviewMesh, MoveTemp(NewChange), LOCTEXT("CardVertexEditChange", "Edit Card"));

		PreviewMesh->ForceRebuildSpatial();
	}

	delete ActiveVertexChange;
	ActiveVertexChange = nullptr;
}







void UGroomCardsEditorTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();
	FTransform Transform = PreviewMesh->GetTransform();
	const FDynamicMesh3* Mesh = PreviewMesh->GetMesh();

	auto DrawCardQuadStripFunc = [&](const UE::GroomQueries::FMeshCardStrip& CardStrip)
	{
		for (FIndex4i Quad : CardStrip.QuadLoops)
		{
			for (int32 j = 0; j < 4; ++j)
			{
				FVector Pos0 = (FVector)Mesh->GetVertex(Quad[j]);
				FVector Pos1 = (FVector)Mesh->GetVertex(Quad[(j + 1) % 4]);
				const FLinearColor& UseColor = (j == 0 || j == 2) ? FLinearColor::Green : FLinearColor::Blue;
				PDI->DrawLine(Transform.TransformPosition(Pos0), Transform.TransformPosition(Pos1),
					UseColor, SDPG_Foreground, 1.0f, 0.0f, true);
			}
		}
	};

	if (CardMeshSelectionMechanic->HasSelection())
	{
		for (int32 SelectedGroup : CardMeshSelectionMechanic->GetActiveSelection().SelectedGroupIDs)
		{
			DrawCardQuadStripFunc(EditableCardSet->CardStrips->FindStripForGroup(SelectedGroup));
		}
	}


	if (ControlPointsMechanic != nullptr)
	{
		ControlPointsMechanic->Render(RenderAPI);
	}

}



void UGroomCardsEditorTool::UpdateLineSet()
{
}




void UGroomCardsEditorTool::InitializeMesh()
{
	EditableCardSet = MakePimpl<FEditableGroomCardSet>();
	EditableCardSet->Initialize(UE::ToolTarget::GetMeshDescription(Target));
	const FDynamicMesh3* CardsMesh = EditableCardSet->FullCardMesh.Get();

	PreviewMesh->UpdatePreview(CardsMesh);

	RecomputeTopology();

	UpdateLineSet();

	if (CardsMesh->TriangleCount() == 0)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("OnNoCardsMesage", "Selected LOD does not contain any Card geometry"),
			EToolMessageLevel::UserWarning);
	}
	else
	{
		GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserWarning);
	}

	InfoProperties->NumCards = EditableCardSet->CardStrips->CardStrips.Num();
	InfoProperties->NumVertices = EditableCardSet->FullCardMesh->VertexCount();
	InfoProperties->NumTriangles = EditableCardSet->FullCardMesh->TriangleCount();
}











void UGroomCardsEditorTool::RequestAction(EEditGroomCardsToolActions ActionType)
{
	if (PendingAction != EEditGroomCardsToolActions::NoAction)
	{
		return;
	}

	PendingAction = ActionType;
}







void UGroomCardsEditorTool::ApplyDeleteAction()
{
	if (CardMeshSelectionMechanic->HasSelection() == false)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("OnDeleteFailedMessage", "Cannot Delete Current Selection"),
			EToolMessageLevel::UserWarning);
		return;
	}

	const FGroupTopologySelection& Selection = CardMeshSelectionMechanic->GetActiveSelection();

	TUniquePtr<FMeshChange> MeshChange = 
		PreviewMesh->TrackedEditMesh([&](FDynamicMesh3& EditMesh, FDynamicMeshChangeTracker& ChangeTracker)
	{
		FDynamicMeshEditor Editor(&EditMesh);
		for (int32 SelectedGroup : Selection.SelectedGroupIDs)
		{
			const UE::GroomQueries::FMeshCardStrip& CardStrip = EditableCardSet->CardStrips->FindStripForGroup(SelectedGroup);

			ChangeTracker.SaveTriangles(CardStrip.Triangles, true);
			Editor.RemoveTriangles(CardStrip.Triangles, true);
		}
	});
	
	// emit undo
	FGroupTopologySelection NewSelection;
	CompleteMeshEditChange(LOCTEXT("StripDeleteChange", "Delete"), MoveTemp(MeshChange), NewSelection);
}




void UGroomCardsEditorTool::RecomputeTopology()
{
	const FDynamicMesh3* Mesh = PreviewMesh->GetMesh();

	Topology = MakeUnique<FGroupTopology>(Mesh, false);
	// we already know the topology so this is very inefficient!
	Topology->RebuildTopology();

	// update selection mechanic
	CardMeshSelectionMechanic->Initialize(
		PreviewMesh->GetMesh(), PreviewMesh->GetTransform(), TargetWorld,
		Topology.Get(),
		[this]() { return PreviewMesh->GetSpatial(); }
	);
}



void UGroomCardsEditorTool::OnPreviewMeshChanged()
{
	CardMeshSelectionMechanic->NotifyMeshChanged(false);
}


void UGroomCardsEditorTool::AfterTopologyEdit()
{
	bWasTopologyEdited = true;
	CardMeshSelectionMechanic->NotifyMeshChanged(true);

	RecomputeTopology();
}




void UGroomCardsEditorTool::CompleteMeshEditChange(
	const FText& TransactionLabel,
	TUniquePtr<FToolCommandChange> EditChange,
	const FGroupTopologySelection& OutputSelection)
{
	// open top-level transaction
	GetToolManager()->BeginUndoTransaction(TransactionLabel);

	// is this the right place? seems like maybe it needs to happen before we actually do mesh edit...
	CompleteActiveCardEditAndEmitChanges();

	// clear current selection
	CardMeshSelectionMechanic->BeginChange();
	CardMeshSelectionMechanic->ClearSelection();
	GetToolManager()->EmitObjectChange(CardMeshSelectionMechanic, CardMeshSelectionMechanic->EndChange(), LOCTEXT("GroomCardsEditChangeClearSelection", "ClearSelection"));

	// emit the pre-edit change
	GetToolManager()->EmitObjectChange(this, MakeUnique<FEditGroomCardsTopologyPreEditChange>(), LOCTEXT("GroomCardsEditChangePreEdit", "PreEdit"));

	// emit the mesh change
	GetToolManager()->EmitObjectChange(PreviewMesh, MoveTemp(EditChange), TransactionLabel);

	// emit the post-edit change
	GetToolManager()->EmitObjectChange(this, MakeUnique<FEditGroomCardsTopologyPostEditChange>(), TransactionLabel);
	// call this (PostEditChange will do this)
	AfterTopologyEdit();
	// increment topology-change counter
	ModifiedTopologyCounter++;

	// set output selection
	if (OutputSelection.IsEmpty() == false)
	{
		CardMeshSelectionMechanic->BeginChange();
		CardMeshSelectionMechanic->SetSelection(OutputSelection);
		GetToolManager()->EmitObjectChange(CardMeshSelectionMechanic, CardMeshSelectionMechanic->EndChange(), LOCTEXT("PolyMeshExtrudeChangeSetSelection", "SetSelection"));
	}

	// complete the transaction
	GetToolManager()->EndUndoTransaction();

	// clean up preview mesh, hiding of things, etc
	//DynamicMeshComponent->SetSecondaryBuffersVisibility(true);

	CurrentOperationTimestamp++;
}







void FEditGroomCardsTopologyPreEditChange::Apply(UObject* Object)
{
}
void FEditGroomCardsTopologyPreEditChange::Revert(UObject* Object)
{
	Cast<UGroomCardsEditorTool>(Object)->AfterTopologyEdit();
	Cast<UGroomCardsEditorTool>(Object)->ModifiedTopologyCounter--;
}
FString FEditGroomCardsTopologyPreEditChange::ToString() const
{
	return TEXT("FEditGroomCardsTopologyPreEditChange");
}


void FEditGroomCardsTopologyPostEditChange::Apply(UObject* Object)
{
	Cast<UGroomCardsEditorTool>(Object)->AfterTopologyEdit();
	Cast<UGroomCardsEditorTool>(Object)->ModifiedTopologyCounter++;
}
void FEditGroomCardsTopologyPostEditChange::Revert(UObject* Object)
{
}
FString FEditGroomCardsTopologyPostEditChange::ToString() const
{
	return TEXT("FEditGroomCardsTopologyPostEditChange");
}




void FBeginGroomCardsEditChange::Apply(UObject* Object)
{
	Cast<UGroomCardsEditorTool>(Object)->BeginCardEdit(CardGroupID);
}
void FBeginGroomCardsEditChange::Revert(UObject* Object)
{
	Cast<UGroomCardsEditorTool>(Object)->EndCardEdit();
}
FString FBeginGroomCardsEditChange::ToString() const
{
	return TEXT("FBeginGroomCardsEditChange");
}



void FEndGroomCardsEditChange::Apply(UObject* Object)
{
	Cast<UGroomCardsEditorTool>(Object)->EndCardEdit();
}
void FEndGroomCardsEditChange::Revert(UObject* Object)
{
	Cast<UGroomCardsEditorTool>(Object)->RestoreCardEdit(CardEdit.Get());
}
FString FEndGroomCardsEditChange::ToString() const
{
	return TEXT("FBeginGroomCardsEditChange");
}




#undef LOCTEXT_NAMESPACE
