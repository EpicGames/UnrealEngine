// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomQueryUtil.h"

#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "Selections/MeshConnectedComponents.h"
#include "FaceGroupUtil.h"
#include "HairCardsBuilder.h"

using namespace UE::Geometry;

const UE::GroomQueries::FMeshCardStrip& UE::GroomQueries::FMeshCardStripSet::FindStripForGroup(int32 GroupIdx) const
{
	int32 CardIdx = TriGroupToCardIndex[GroupIdx];
	return CardStrips[CardIdx];
}

UE::GroomQueries::FMeshCardStrip& UE::GroomQueries::FMeshCardStripSet::FindStripForGroup(int32 GroupIdx)
{
	int32 CardIdx = TriGroupToCardIndex[GroupIdx];
	return CardStrips[CardIdx];
}

void UE::GroomQueries::ExtractAllHairCards(AGroomActor* GroomActor,
	int32 LODIndex,
	FDynamicMesh3& MeshOut,
	FMeshCardStripSet* CardsOut)
{
	check(GroomActor->GetGroomComponent());
	check(GroomActor->GetGroomComponent()->GroomAsset);
	UGroomAsset* Asset = GroomActor->GetGroomComponent()->GroomAsset;

	MeshOut.EnableTriangleGroups(0);
	MeshOut.EnableAttributes();
	FDynamicMeshUVOverlay* UVs = MeshOut.Attributes()->PrimaryUV();
	FDynamicMeshNormalOverlay* Normals = MeshOut.Attributes()->PrimaryNormals();

	int32 NumHairGroups = Asset->GetHairGroupsInfo().Num();
	for (int32 GroupIdx = 0; GroupIdx < NumHairGroups; ++GroupIdx)
	{
		const FHairGroupInfo& GroupInfo = Asset->GetHairGroupsInfo()[GroupIdx];
		const FHairGroupPlatformData& GroupData = Asset->GetHairGroupsPlatformData()[GroupIdx];

		int32 NumCardsLODs = GroupData.Cards.LODs.Num();
		if (LODIndex >= NumCardsLODs)
		{
			continue;
		}

		UStaticMesh* StaticMesh = nullptr;
		for (const FHairGroupsCardsSourceDescription& Desc : Asset->GetHairGroupsCards())
		{
			if (Desc.GroupIndex == GroupIdx && Desc.LODIndex == LODIndex)
			{
				StaticMesh = Desc.ImportedMesh;
			}
		}
		
		if (!StaticMesh)
		{
			continue;
		}

		FHairCardsDatas CardData;
		FHairStrandsDatas DummyStrandsData;
		if (!FHairCardsBuilder::ExtractCardsData(StaticMesh, DummyStrandsData, CardData))
		{
			continue;
		}
		const FHairCardsGeometry& CardGeo = CardData.Cards;

		const uint32 CardUVIndex = FHairCardsGeometry::FUVChannel::GetCardUVIndex();
		int32 NumVerts = CardGeo.GetNumVertices();
		TArray<int32> MapV, MapN, MapU;
		MapV.SetNum(NumVerts);
		MapU.SetNum(NumVerts);
		MapN.SetNum(NumVerts);
		for (int32 k = 0; k < NumVerts; ++k)
		{
			int32 NewVID = MeshOut.AppendVertex((FVector3d)CardGeo.Positions[k]);
			MapV[k] = NewVID;

			int32 NewNormID = Normals->AppendElement((FVector3f)CardGeo.Normals[k]);
			MapN[k] = NewNormID;

			int32 NewUVID = UVs->AppendElement( FVector2f(CardGeo.UVs[CardUVIndex].UV[k].X, CardGeo.UVs[CardUVIndex].UV[k].Y) );
			MapU[k] = NewUVID;
		}

		int32 NumCards = CardGeo.IndexOffsets.Num();

		TArray<int32> CardTriGroups;
		CardTriGroups.SetNum(NumCards);
		for (int32 k = 0; k < NumCards; ++k)
		{
			CardTriGroups[k] = MeshOut.AllocateTriangleGroup();
		}

		int32 NumTris = CardGeo.GetNumTriangles();

		TArray<int32> CardTriVertIndexToGroup;
		CardTriVertIndexToGroup.SetNum(3 * NumTris);
		for (int32 k = 0; k < NumCards; ++k)
		{
			int32 Offset = CardGeo.IndexOffsets[k];
			int32 Count = CardGeo.IndexCounts[k];
			for (int32 j = 0; j < Count; ++j)
			{
				CardTriVertIndexToGroup[Offset + j] = CardTriGroups[k];
			}
		}

		if (CardsOut != nullptr)
		{
			CardsOut->CardStrips.Reserve(CardsOut->CardStrips.Num() + NumCards);
			for (int32 k = 0; k < NumCards; ++k)
			{
				int32 NewCardStripIndex = CardsOut->CardStrips.Num();
				FMeshCardStrip& NewCardStrip = CardsOut->CardStrips.Emplace_GetRef();
				NewCardStrip.GroupID = CardTriGroups[k];
				NewCardStrip.SourceInfo.GroupIndex = GroupIdx;
				NewCardStrip.SourceInfo.LODIndex = LODIndex;
				NewCardStrip.SourceInfo.CardIndex = k;
				NewCardStrip.SourceInfo.IndexOffset = CardGeo.IndexOffsets[k];
				NewCardStrip.SourceInfo.IndexCount = CardGeo.IndexCounts[k];

				CardsOut->TriGroupToCardIndex.Add(NewCardStrip.GroupID, NewCardStripIndex);
			}
		}

		for (int32 k = 0; k < NumTris; ++k)
		{
			int32 GroupID = CardTriVertIndexToGroup[3 * k];
			FIndex3i CardTri(CardGeo.Indices[3 * k], CardGeo.Indices[3 * k + 1], CardGeo.Indices[3 * k + 2]);

			int32 NewTID = MeshOut.AppendTriangle(MapV[CardTri.A], MapV[CardTri.B], MapV[CardTri.C], GroupID);
			Normals->SetTriangle(NewTID, FIndex3i(MapN[CardTri.A], MapN[CardTri.B], MapN[CardTri.C]));
			UVs->SetTriangle(NewTID, FIndex3i(MapU[CardTri.A], MapU[CardTri.B], MapU[CardTri.C]));

			if (CardsOut)
			{
				int32 CardStripIndex = CardsOut->TriGroupToCardIndex[GroupID];
				CardsOut->CardStrips[CardStripIndex].Triangles.Add(NewTID);
			}
		}
	}

}




void UE::GroomQueries::ExtractAllHairCards(
	const FMeshDescription* SourceMeshIn,
	FDynamicMesh3& MeshOut,
	FMeshCardStripSet& CardsOut)
{
	FMeshDescriptionToDynamicMesh Converter;
	Converter.Convert(SourceMeshIn, MeshOut);

	MeshOut.EnableTriangleGroups();

	FMeshConnectedComponents Components(&MeshOut);
	Components.FindConnectedTriangles();
	int32 NumCards = Components.Num();


	CardsOut.CardStrips.Reserve(NumCards);
	for (int32 k = 0; k < NumCards; ++k)
	{
		int32 CardTriGroup = MeshOut.AllocateTriangleGroup();

		FaceGroupUtil::SetGroupID(MeshOut, Components.GetComponent(k).Indices, CardTriGroup);

		int32 NewCardStripIndex = CardsOut.CardStrips.Num();
		FMeshCardStrip& NewCardStrip = CardsOut.CardStrips.Emplace_GetRef();

		NewCardStrip.GroupID = CardTriGroup;
		NewCardStrip.SourceInfo.GroupIndex = 0;
		NewCardStrip.SourceInfo.LODIndex = 0;
		NewCardStrip.SourceInfo.CardIndex = k;
		NewCardStrip.SourceInfo.IndexOffset = -1;
		NewCardStrip.SourceInfo.IndexCount = -1;

		CardsOut.TriGroupToCardIndex.Add(NewCardStrip.GroupID, NewCardStripIndex);

		NewCardStrip.Triangles = Components.GetComponent(k).Indices;

		// if we have an odd number of tris we are going to discard the last triangle and hope for the best
		if (ensure(NewCardStrip.Triangles.Num() % 2 == 0) == false)
		{
			NewCardStrip.Triangles.SetNum(NewCardStrip.Triangles.Num() - 1);
		}
	}
}



static bool QuadsAreConnected(const FDynamicMesh3& Mesh, FIndex2i Quad1, FIndex2i Quad2)
{
	for (int32 j = 0; j < 2; ++j)
	{
		int32 Tri = Quad1[j];
		FIndex3i TriNbrsTris = Mesh.GetTriNeighbourTris(Quad1[j]);
		if (IndexUtil::FindTriIndex(Quad2.A, TriNbrsTris) != IndexConstants::InvalidID ||
			IndexUtil::FindTriIndex(Quad2.B, TriNbrsTris) != IndexConstants::InvalidID)
		{
			return true;
		}
	}
	return false;
}

static FIndex3i FindSharedQuadEdge(const FDynamicMesh3& Mesh, FIndex4i QuadA, FIndex4i QuadB)
{
	for (int32 j = 0; j < 4; ++j)
	{
		int32 A = QuadA[j], B = QuadA[(j + 1) % 4];
		for (int32 k = 0; k < 4; ++k)
		{
			int32 C = QuadB[k], D = QuadB[(k + 1) % 4];
			if (A == C && B == D)
			{
				return FIndex3i(j, k, 1);
			}
			else if (A == D && B == C)
			{
				return FIndex3i(j, k, -1);
			}
		}
	}
	return FIndex3i(-1, -1, -1);
}


static bool QuadConnectedToTri(const FDynamicMesh3& Mesh, int32 TriangleID, FIndex2i Quad2)
{
	FIndex3i TriNbrsTris = Mesh.GetTriNeighbourTris(TriangleID);
	if (IndexUtil::FindTriIndex(Quad2.A, TriNbrsTris) != IndexConstants::InvalidID ||
		IndexUtil::FindTriIndex(Quad2.B, TriNbrsTris) != IndexConstants::InvalidID)
	{
		return true;
	}
	return false;
}



void UE::GroomQueries::ExtractCardQuads(
	const FDynamicMesh3& Mesh,
	UE::GroomQueries::FMeshCardStripSet& CardsInOut)
{
	FDynamicMesh3 TmpMesh;
	TmpMesh.Copy(Mesh, false, false, false, false);

	int32 NumCards = CardsInOut.CardStrips.Num();
	for (int32 ci = 0; ci < NumCards; ++ci)
	{
		FMeshCardStrip& CardStrip = CardsInOut.CardStrips[ci];
		bool bIsValidCard = true;

		TArray<int32> RemainingTris(CardStrip.Triangles);
		while (RemainingTris.Num() > 1)
		{
			bool bIsLastQuad = (RemainingTris.Num() < 4);
			int32 LastRemainingTrisCount = RemainingTris.Num();

			for (int32 tid : RemainingTris)
			{
				FIndex3i TriEdges = TmpMesh.GetTriEdges(tid);
				FIndex3i TriBdryEdgeFlag(
					TmpMesh.IsBoundaryEdge(TriEdges.A) ? 1 : 0,
					TmpMesh.IsBoundaryEdge(TriEdges.B) ? 1 : 0,
					TmpMesh.IsBoundaryEdge(TriEdges.C) ? 1 : 0);
				bool bFoundMatch = false;
				if ( (TriBdryEdgeFlag.A + TriBdryEdgeFlag.B + TriBdryEdgeFlag.C) == 2 )
				{
					int32 SharedEdgeIdx = IndexUtil::FindTriIndex(0, TriBdryEdgeFlag);
					int32 SharedEdge = TriEdges[SharedEdgeIdx];
					FIndex2i NbrEdgeT = TmpMesh.GetEdgeT(SharedEdge);
					check(NbrEdgeT.B != IndexConstants::InvalidID);
					int32 OtherTID = (NbrEdgeT.A == tid) ? NbrEdgeT.B : NbrEdgeT.A;

					FIndex2i CardTris(tid, OtherTID);

					FIndex3i Tri1 = TmpMesh.GetTriangle(CardTris.A);
					FIndex3i Tri2 = TmpMesh.GetTriangle(CardTris.B);

					// can we use this to sort out the correct cycle shift
					FIndex2i SharedEdgeV = TmpMesh.GetEdgeV(SharedEdge);
					FIndex2i OpposingV = TmpMesh.GetEdgeOpposingV(SharedEdge);

					// need to cycle the first triangle verts correctly so that when we add fourth vertex we don't create an hourglass quad
					FIndex4i Quad(Tri1.A, Tri1.B, Tri1.C, 0);		// SharedEdgeIdx == 2 case
					if (SharedEdgeIdx == 0)
					{
						Quad = FIndex4i(Tri1.B, Tri1.C, Tri1.A, 0);
					}
					else if (SharedEdgeIdx == 1)
					{
						Quad = FIndex4i(Tri1.C, Tri1.A, Tri1.B, 0);
					}

					// figure out the fourth vertex, ie which of the vertices of tri2 was not in tri1
					if (IndexUtil::FindTriIndex(Tri2.A, Tri1) == FDynamicMesh3::InvalidID)
					{
						Quad.D = Tri2.A;
					}
					else if (IndexUtil::FindTriIndex(Tri2.B, Tri1) == FDynamicMesh3::InvalidID)
					{
						Quad.D = Tri2.B;
					}
					else
					{
						Quad.D = Tri2.C;
					}
					
					int32 FreeVertIdx = -1;
					for (int32 j = 0; j < 3 && FreeVertIdx == -1; ++j)
					{
						FreeVertIdx = (IndexUtil::FindTriIndex(Quad[j], Tri2) == FDynamicMesh3::InvalidID) ? j : -1;
					}

					// attach new quad to start or end of strip depending on which tris were adjacent (does this work??)
					if (CardStrip.QuadTriPairs.Num() == 0 || QuadConnectedToTri(Mesh, CardStrip.QuadTriPairs.Last().B, CardTris))
					{
						CardStrip.QuadTriPairs.Add(CardTris);
						CardStrip.QuadLoops.Add(Quad);
					}
					else if (QuadConnectedToTri(Mesh, CardStrip.QuadTriPairs[0].A, CardTris))
					{
						CardStrip.QuadTriPairs.Insert(CardTris, 0);
						CardStrip.QuadLoops.Insert(Quad, 0);
					}
					else
					{
						check(false);
					}

					RemainingTris.RemoveSwap(tid, EAllowShrinking::No);
					RemainingTris.RemoveSwap(OtherTID, EAllowShrinking::No);
					TmpMesh.RemoveTriangle(tid, false);

					FIndex3i OtherNbrTris = TmpMesh.GetTriNeighbourTris(OtherTID);
					TmpMesh.RemoveTriangle(OtherTID, false);

					// figure out the next connected tri in the strip, and reorder array so that we start there next iteration.
					// this allows the quads to be processed in-order
					for (int32 j = 0; j < 3; ++j)
					{
						if (OtherNbrTris[j] != IndexConstants::InvalidID && OtherNbrTris[j] != tid && RemainingTris.Contains(OtherNbrTris[j]))
						{
							int32 OtherIdx = RemainingTris.IndexOfByKey(OtherNbrTris[j]);
							Swap(RemainingTris[0], RemainingTris[OtherIdx]);
						}
					}
					bFoundMatch = true;
				}

				if (bFoundMatch)		// break out of loop over RemainingTris and go to next
				{
					break;
				}
			}

			// if we did not remove any triangles this loop, this is not a valid card
			if (LastRemainingTrisCount == RemainingTris.Num())
			{
				bIsValidCard = false;
				break;
			}
		}

		if (!bIsValidCard)
		{
			continue;
		}

		// Output of code above has each quad oriented properly, but the vertex cycle is not consistent.
		// So now we walk down the strip and orient each quad in order [A,B,C,D] such that the centerline
		// of the strip passes through Midpoint(A,B) and Midpoint(C,D), and the "side" edges are (B,C) and (D,A).
		// We can figure out the correct ordering based on which edge is currently shared w/ the next Quad,
		// except at the last quad, which has no next so we have to use the Previous
		int32 NumQuads = CardStrip.QuadLoops.Num();
		for (int32 qi = 0; qi < NumQuads && NumQuads > 1; ++qi)
		{
			bool bIsLast = (qi == NumQuads - 1);
			FIndex4i CurQuad = CardStrip.QuadLoops[qi];
			FIndex4i NextQuad = CardStrip.QuadLoops[bIsLast ? (NumQuads-2) : (qi+1)];		// use previous for last quad
			FIndex3i SharedEdgeInfo = FindSharedQuadEdge(Mesh, CurQuad, NextQuad);
			if (ensure(SharedEdgeInfo.A != -1))
			{
				SharedEdgeInfo.A = (bIsLast) ? ((SharedEdgeInfo.A + 2) % 4) : SharedEdgeInfo.A;		// cases are reversed for last quad
				if (SharedEdgeInfo.A == 0)
				{
					CardStrip.QuadLoops[qi] = FIndex4i(CurQuad.C, CurQuad.D, CurQuad.A, CurQuad.B);
				}
				else if (SharedEdgeInfo.A == 1)
				{
					CardStrip.QuadLoops[qi] = FIndex4i(CurQuad.D, CurQuad.A, CurQuad.B, CurQuad.C);
				}
				else if (SharedEdgeInfo.A == 3)
				{
					CardStrip.QuadLoops[qi] = FIndex4i(CurQuad.B, CurQuad.C, CurQuad.D, CurQuad.A);
				}
			}
		}

	}
}



void UE::GroomQueries::ExtractCardCurves(
	const FDynamicMesh3& Mesh,
	UE::GroomQueries::FMeshCardStripSet& CardsInOut)
{

	int32 NumCards = CardsInOut.CardStrips.Num();
	for (int32 ci = 0; ci < NumCards; ++ci)
	{
		FMeshCardStrip& CardStrip = CardsInOut.CardStrips[ci];

		// skip strips w/ no quads
		if (CardStrip.QuadLoops.Num() == 0)
		{
			continue;
		}

		//for (FIndex4i Quad : CardStrip.QuadLoops)
		//{
		//	FVector3d Center =
		//		Mesh.GetVertex(Quad.A) + Mesh.GetVertex(Quad.B) + Mesh.GetVertex(Quad.C) + Mesh.GetVertex(Quad.D);
		//	Center /= 4.0;
		//	CardStrip.CardCurve.Add(Center);
		//	CardStrip.CardCurveWidth.Add(0.0);
		//}

		int32 NumQuads = CardStrip.QuadLoops.Num();
		for (int32 k = 0; k <= NumQuads; ++k)
		{
			FVector3d A, B;
			if (k == NumQuads)
			{
				FIndex4i CardQuad = CardStrip.QuadLoops[k-1];
				A = Mesh.GetVertex(CardQuad.C);
				B = Mesh.GetVertex(CardQuad.D);
			}
			else
			{
				FIndex4i CardQuad = CardStrip.QuadLoops[k];
				A = Mesh.GetVertex(CardQuad.A);
				B = Mesh.GetVertex(CardQuad.B);
			}
			CardStrip.CardCurve.Add( (A+B) * 0.5 );
			CardStrip.CardCurveWidth.Add((B - A).Length() * 0.5);
		}


		int32 NumVertices = CardStrip.CardCurve.Num();
		for (int32 k = 0; k < NumVertices; ++k)
		{
			FIndex2i AdjacentVerts = CardStrip.GetCurvePointVertices(k);
			FVector3d Left = Mesh.GetVertex(AdjacentVerts.A);
			FVector3d Right = Mesh.GetVertex(AdjacentVerts.B);
			FVector3d Center = CardStrip.CardCurve[k];

			FFrame3d CardFrame(Center);
			CardFrame.AlignAxis(0, UE::Geometry::Normalized(Right - Center));

			FVector3d Y;
			if (k == 0)
			{
				FVector3d Forward = CardStrip.CardCurve[k + 1];
				Y = UE::Geometry::Normalized(Forward - Center);
			}
			else if (k == NumVertices - 1)
			{
				FVector3d Back = CardStrip.CardCurve[k-1];
				Y = UE::Geometry::Normalized(Center - Back);
			}
			else
			{
				FVector3d Forward = CardStrip.CardCurve[k + 1];
				FVector3d Back = CardStrip.CardCurve[k - 1];
				Y = UE::Geometry::Normalized(Forward - Back);
			}

			CardFrame.ConstrainedAlignAxis(1, Y, CardFrame.X());
			CardStrip.CardCurveFrames.Add(CardFrame);
		}
	}

}









void UE::GroomQueries::ProcessHairCurves(AGroomActor* GroomActor,
	bool bUseGuides,
	TFunctionRef<void(const TArrayView<FVector3f>& Positions, const TArrayView<float>& Radii)> HairCurveFunc)
{
	check(GroomActor->GetGroomComponent());
	check(GroomActor->GetGroomComponent()->GroomAsset);
	UGroomAsset* Asset = GroomActor->GetGroomComponent()->GroomAsset;

	int32 NumHairGroups = Asset->GetHairGroupsInfo().Num();
	for (int32 GroupIdx = 0; GroupIdx < NumHairGroups; ++GroupIdx)
	{
		const FHairGroupInfo& GroupInfo = Asset->GetHairGroupsInfo()[GroupIdx];
		const FHairGroupPlatformData& GroupData = Asset->GetHairGroupsPlatformData()[GroupIdx];

		//int32 NumCurves = GroupInfo.NumCurves;
		//int32 NumGuides = GroupInfo.NumGuides;

		FHairStrandsDatas StrandsData;
		FHairStrandsDatas GuidesData;
		Asset->GetHairStrandsDatas(GroupIdx, StrandsData, GuidesData);

		//const FHairStrandsDatas& GroupStrandData = (bUseGuides) ? GroupData.HairSimulationData : GroupData.HairRenderData;
		const FHairStrandsDatas& GroupStrandData = (bUseGuides) ? GuidesData : StrandsData;

		const FHairStrandsPoints& GroupStrandPoints = GroupStrandData.StrandsPoints;
		const TArray<FVector3f>& Positions = GroupStrandPoints.PointsPosition;
		const TArray<float>& Radii = GroupStrandPoints.PointsRadius;

		const FHairStrandsCurves& GroupStrandCurves = GroupStrandData.StrandsCurves;
		const TArray<uint16>& CurvesCounts = GroupStrandCurves.CurvesCount;
		const TArray<uint32>& CurvesOffsets = GroupStrandCurves.CurvesOffset;

		int32 NumCurves = CurvesCounts.Num();
		for (int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
		{
			int32 Count = (int32)CurvesCounts[CurveIndex];
			int32 Offset = (int32)CurvesOffsets[CurveIndex];

			TArrayView<FVector3f> CurvePositions = TArrayView<FVector3f>((FVector3f*)&Positions[Offset], Count);
			TArrayView<float> CurveRadii = TArrayView<float>((float*)&Radii[Offset], Count);

			HairCurveFunc(CurvePositions, CurveRadii);

			//for (int32 k = 0; k < Count - 1; ++k)
			//{
			//	FSkeletalImplicitLine3d Line = { FSegment3d(FVector3d(Positions[Offset + k]), FVector3d(Positions[Offset + k + 1])), Radii[Offset + k] };
			//	//Line.Radius = 5.0;
			//	Lines.Add(Line);
			//}
		}

		//FString LinesName = FString::Printf(TEXT("STRANDGROUP_%d"), GroupIdx);
		//PreviewGeom->CreateOrUpdateLineSet(LinesName, GroupStrandCurves.Num(),
		//	[&](int32 CurveIndex, TArray<FRenderableLine>& LinesOut)
		//{
		//	int32 Count = (int32)CurvesCounts[CurveIndex];
		//	int32 Offset = (int32)CurvesOffsets[CurveIndex];
		//	for (int32 k = 0; k < Count - 1; ++k)
		//	{
		//		LinesOut.Add(FRenderableLine(Positions[Offset + k], Positions[Offset + k + 1], FColor::Red, 0.1f));
		//	}
		//});
	}

}