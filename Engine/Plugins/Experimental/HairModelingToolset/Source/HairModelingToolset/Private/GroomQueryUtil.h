// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "GroomActor.h"
#include "GroomComponent.h"
#include "GroomAsset.h"

#include "DynamicMesh/DynamicMesh3.h"



namespace UE
{
	namespace GroomQueries
	{
		using namespace UE::Geometry;

		// source card information
		struct FCardSourceInfo
		{
			int32 GroupIndex;		// groom asset group
			int32 LODIndex;			// groom asset LOD
			int32 CardIndex;		// index of card into card arrays
			int32 IndexOffset;		// index offset into card triangle buffer
			int32 IndexCount;		// number of indices after offset that belong to this card
		};

		struct FMeshCardStrip
		{
			FCardSourceInfo SourceInfo;

			int32 GroupID;
			TArray<int32> Triangles;

			TArray<FIndex2i> QuadTriPairs;
			TArray<FIndex4i> QuadLoops;

			TArray<FVector3d> CardCurve;
			TArray<FFrame3d> CardCurveFrames;
			TArray<double> CardCurveWidth;

			FIndex2i GetCurvePointVertices(int32 Index) const
			{
				if (Index >= (CardCurve.Num() - 1))
				{
					Index = QuadLoops.Num() - 1;
					return FIndex2i(QuadLoops[Index].D, QuadLoops[Index].C);
				}
				else
				{
					return FIndex2i(QuadLoops[Index].A, QuadLoops[Index].B);
				}
			}
		};

		struct FMeshCardStripSet
		{
			TArray<FMeshCardStrip> CardStrips;

			// map from mesh triangle GroupIDs to CardStrips array index
			TMap<int32,int32> TriGroupToCardIndex;

			const FMeshCardStrip& FindStripForGroup(int32 GroupIdx) const;
			FMeshCardStrip& FindStripForGroup(int32 GroupIdx);
		};

		void ExtractAllHairCards(AGroomActor* GroomActor,
			int32 LODIndex,
			FDynamicMesh3& MeshOut,
			FMeshCardStripSet* CardsOut = nullptr);

		void ExtractAllHairCards(
			const FMeshDescription* SourceMeshIn,
			FDynamicMesh3& MeshOut,
			FMeshCardStripSet& CardsOut);

		void ExtractCardQuads(
			const FDynamicMesh3& Mesh,
			FMeshCardStripSet& CardsInOut);

		void ExtractCardCurves(
			const FDynamicMesh3& Mesh,
			FMeshCardStripSet& CardsInOut);

		void ProcessHairCurves(AGroomActor* GroomActor,
			bool bUseGuides,
			TFunctionRef<void(const TArrayView<FVector3f>& Positions, const TArrayView<float>& Radii)> HairCurveFunc);


	}
}






