// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroomToMeshTool.h"
#include "InteractiveToolManager.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "ToolBuilderUtil.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Spatial/FastWinding.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMeshEditor.h"
#include "Util/ColorConstants.h"
#include "ToolSetupUtil.h"
#include "Selection/ToolSelectionUtil.h"
#include "ModelingObjectsCreationAPI.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Selections/MeshFaceSelection.h"
#include "Selections/MeshVertexSelection.h"
#include "Selections/MeshConnectedComponents.h"
#include "DynamicMeshEditor.h"
#include "MeshBoundaryLoops.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "Parameterization/MeshUVTransforms.h"

#include "DynamicMesh/DynamicPointSet3.h"
#include "DynamicMesh/DynamicVerticesOctree3.h"
#include "DynamicMesh/DynamicAttribute.h"
#include "DynamicMesh/DynamicVertexAttribute.h"

#include "Implicit/CachingMeshSDF.h"
#include "Implicit/GridInterpolant.h"
#include "Implicit/ImplicitFunctions.h"
#include "Implicit/Morphology.h"
#include "Generators/MarchingCubes.h"

#include "MeshSimplification.h"
#include "SmoothingOps/CotanSmoothingOp.h"

#include "GroomActor.h"
#include "GroomComponent.h"
#include "GroomAsset.h"
#include "HairDescription.h"

#include "ParameterizationOps/UVLayoutOp.h"

#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "MeshDescriptionToDynamicMesh.h"

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGroomToMeshTool"


/*
 * ToolBuilder
 */

bool UGroomToMeshToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return ToolBuilderUtil::CountSelectedActorsOfType<AGroomActor>(SceneState) == 1;
}

UInteractiveTool* UGroomToMeshToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UGroomToMeshTool* NewTool = NewObject<UGroomToMeshTool>(SceneState.ToolManager);

	NewTool->SetWorld(SceneState.World);
	AGroomActor* Groom = ToolBuilderUtil::FindFirstActorOfType<AGroomActor>(SceneState);
	check(Groom != nullptr);
	NewTool->SetSelection(Groom);

	return NewTool;
}




/*
 * Tool
 */
UGroomToMeshTool::UGroomToMeshTool()
{
	SetToolDisplayName(LOCTEXT("GroomToMeshToolName", "Groom to Mesh"));
}


void UGroomToMeshTool::SetSelection(AGroomActor* Groom)
{
	TargetGroom = Groom;
}


void UGroomToMeshTool::Setup()
{
	UInteractiveTool::Setup();

	PreviewMesh = NewObject<UPreviewMesh>(this);
	PreviewMesh->bBuildSpatialDataStructure = false;
	PreviewMesh->CreateInWorld(TargetGroom->GetWorld(), FTransform::Identity);
	PreviewMesh->SetTransform(TargetGroom->GetActorTransform());

	MeshMaterial = ToolSetupUtil::GetDefaultSculptMaterial(GetToolManager());
	UVMaterial = ToolSetupUtil::GetUVCheckerboardMaterial(50.0);
	PreviewMesh->SetMaterial(MeshMaterial);

	PreviewGeom = NewObject<UPreviewGeometry>(this);
	PreviewGeom->CreateInWorld(TargetGroom->GetWorld(), FTransform::Identity);
	PreviewGeom->GetActor()->SetActorTransform(TargetGroom->GetActorTransform());

	Settings = NewObject<UGroomToMeshToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);

	Settings->WatchProperty(Settings->VoxelCount, [this](int32) { bResultValid = false; });
	Settings->WatchProperty(Settings->BlendPower, [this](float) { bResultValid = false; });
	Settings->WatchProperty(Settings->RadiusScale, [this](float) { bResultValid = false; });

	Settings->WatchProperty(Settings->bApplyMorphology, [this](bool) { bResultValid = false; });
	Settings->WatchProperty(Settings->ClosingDist, [this](float) { bResultValid = false; });
	Settings->WatchProperty(Settings->OpeningDist, [this](float) { bResultValid = false; });

	Settings->WatchProperty(Settings->bClipToHead, [this](bool) { bResultValid = false; });
	Settings->WatchProperty(Settings->ClipMeshActor, [this](TSoftObjectPtr<AStaticMeshActor>) { bResultValid = false; });

	Settings->WatchProperty(Settings->bSmooth, [this](bool) { bResultValid = false; });
	Settings->WatchProperty(Settings->Smoothness, [this](float) { bResultValid = false; });
	Settings->WatchProperty(Settings->VolumeCorrection, [this](float) { bResultValid = false; });

	Settings->WatchProperty(Settings->UVMode, [this](EGroomToMeshUVMode) { bResultValid = false; });

	Settings->WatchProperty(Settings->bSimplify, [this](bool) { bResultValid = false; });
	Settings->WatchProperty(Settings->VertexCount, [this](int32) { bResultValid = false; });
	
	Settings->WatchProperty(Settings->bShowSideBySide, [this](bool) { bResultValid = false; });
	Settings->WatchProperty(Settings->bShowGuides, [this](bool) { bResultValid = false; });
	Settings->WatchProperty(Settings->bShowUVs, [this](bool) { bVisualizationChanged = true; });

	bResultValid = false;
	bVisualizationChanged = true;

	GetToolManager()->DisplayMessage( 
		LOCTEXT("OnStartTool", "Convert a Groom to a Static Mesh"),
		EToolMessageLevel::UserNotification);
}



void UGroomToMeshTool::Shutdown(EToolShutdownType ShutdownType)
{
	Settings->SaveProperties(this);

	FTransformSRT3d Transform(PreviewMesh->GetTransform());
	PreviewMesh->Disconnect();
	PreviewMesh = nullptr;

	PreviewGeom->Disconnect();
	PreviewGeom = nullptr;

	if (ShutdownType == EToolShutdownType::Accept )
	{
		UMaterialInterface* UseMaterial = UMaterial::GetDefaultMaterial(MD_Surface);

		FString NewName = TargetGroom.IsValid() ?
			FString::Printf(TEXT("%sMesh"), *TargetGroom->GetName()) : TEXT("Groom Mesh");

		GetToolManager()->BeginUndoTransaction(LOCTEXT("CreateMeshGroom", "Groom To Mesh"));

		FCreateMeshObjectParams NewMeshObjectParams;
		NewMeshObjectParams.TargetWorld = TargetWorld;
		NewMeshObjectParams.Transform = (FTransform)Transform;
		NewMeshObjectParams.BaseName = NewName;
		NewMeshObjectParams.Materials.Add(UseMaterial);
		NewMeshObjectParams.SetMesh(&CurrentMesh);
		FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
		if (Result.IsOK() && Result.NewActor != nullptr)
		{
			ToolSelectionUtil::SetNewActorSelection(GetToolManager(), Result.NewActor);
		}

		GetToolManager()->EndUndoTransaction();
	}


}


void UGroomToMeshTool::OnTick(float DeltaTime)
{
	if (bResultValid == false)
	{
		RecalculateMesh();
		bResultValid = true;
	}

	if (bVisualizationChanged)
	{
		PreviewMesh->SetMaterial(Settings->bShowUVs ? UVMaterial : MeshMaterial);
		bVisualizationChanged = false;
	}
}

void UGroomToMeshTool::Render(IToolsContextRenderAPI* RenderAPI)
{
}


bool UGroomToMeshTool::HasAccept() const
{
	return true;
}

bool UGroomToMeshTool::CanAccept() const
{
	return true;
}





class FHairPointSetAttributes : public TDynamicAttributeSetBase<FDynamicPointSet3d>
{
public:
	using FRadiusAttribute = TDynamicVertexAttribute<float, 1, FDynamicPointSet3d>;

	TUniquePtr<FRadiusAttribute> Radius;

	void Initialize(FDynamicPointSet3d* Parent)
	{
		Radius = MakeUnique<FRadiusAttribute>(Parent);

		RegisterExternalAttribute(Radius.Get());
	}

};



void UGroomToMeshTool::RecalculateMesh()
{
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> CurrentResult = UpdateVoxelization();

	// helper to only use NewMesh if it was valid, otherwise ignore it
	auto SelectValidMesh = [](TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InCurrentMesh, TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InNewMesh)
	{
		if (InNewMesh->TriangleCount() > 0)
		{
			return InNewMesh;
		}
		return InCurrentMesh;
	};

	if (Settings->bApplyMorphology && CurrentResult->TriangleCount() > 0)
	{
		CurrentResult = SelectValidMesh(CurrentResult, UpdateMorphology(CurrentResult));
	}

	if (Settings->bClipToHead && CurrentResult->TriangleCount() > 0)
	{
		CurrentResult = SelectValidMesh(CurrentResult, UpdateClipMesh(CurrentResult));
	}

	if (Settings->bSmooth && CurrentResult->TriangleCount() > 0)
	{
		CurrentResult = SelectValidMesh(CurrentResult, UpdateSmoothing(CurrentResult));
	}

	if (Settings->bSimplify && CurrentResult->TriangleCount() > 0)
	{
		CurrentResult = SelectValidMesh(CurrentResult, UpdateSimplification(CurrentResult));
	}

	if (CurrentResult->TriangleCount() > 0)
	{
		CurrentResult = SelectValidMesh(CurrentResult, UpdatePostprocessing(CurrentResult));
	}

	UpdatePreview(CurrentResult);
}








static void ProcessHairCurvePoints(AGroomActor* GroomActor,
	bool bUseGuides,
	TFunctionRef<void(const TArray<FVector3f>& Positions, const TArray<float>& Radii)> HairCurvePointsFunc)
{
	check(GroomActor->GetGroomComponent());
	check(GroomActor->GetGroomComponent()->GroomAsset);
	UGroomAsset* Asset = GroomActor->GetGroomComponent()->GroomAsset;

	int32 NumHairGroups = Asset->GetHairGroupsInfo().Num();
	for (int32 GroupIdx = 0; GroupIdx < NumHairGroups; ++GroupIdx)
	{
		const FHairGroupInfo& GroupInfo = Asset->GetHairGroupsInfo()[GroupIdx];
		const FHairGroupPlatformData& GroupData = Asset->GetHairGroupsPlatformData()[GroupIdx];

		//int32 NumCurves = GroupInfo.NumCurves;
		//int32 NumGuides = GroupInfo.NumGuides;

		//const FHairStrandsDatas& GroupStrandData = (bUseGuides) ? GroupData.HairSimulationData : GroupData.HairRenderData;
		FHairStrandsDatas StrandsData;
		FHairStrandsDatas GuidesData;
		Asset->GetHairStrandsDatas(GroupIdx, StrandsData, GuidesData);
		const FHairStrandsDatas& GroupStrandData = (bUseGuides) ? GuidesData : StrandsData;

		const FHairStrandsPoints& GroupStrandPoints = GroupStrandData.StrandsPoints;
		const TArray<FVector3f>& Positions = GroupStrandPoints.PointsPosition;
		const TArray<float>& Radii = GroupStrandPoints.PointsRadius;

		HairCurvePointsFunc(GroupStrandPoints.PointsPosition, GroupStrandPoints.PointsRadius);

		//int32 NumPoints = Positions.Num();
		//for (int32 k = 0; k < NumPoints; ++k)
		//{
		//	FVector3d Position(Positions[k]);
		//	Bounds.Contain(Position);
		//	int32 NewVID = Points->AppendVertex(Position);
		//	PointAttribs->Radius->SetNewValue(NewVID, &Radii[k]);
		//}

	}
}






static void ProcessHairCurves(AGroomActor* GroomActor,
	bool bUseGuides,
	TFunctionRef<void(const TArrayView<FVector3f>& Positions, const TArrayView<float>& Radii)> HairCurveFunc)
{
	check(GroomActor->GetGroomComponent());
	check(GroomActor->GetGroomComponent()->GroomAsset);
	UGroomAsset* Asset = GroomActor->GetGroomComponent()->GroomAsset;

	int32 NumHairGroups = Asset->GetHairGroupsInfo().Num();
	for (int32 GroupIdx = 0; GroupIdx < NumHairGroups; ++GroupIdx)
	{
		const FHairGroupInfo& GroupInfo = Asset->GetHairGroupsInfo()[GroupIdx];
		const FHairGroupPlatformData& GroupData = Asset->GetHairGroupsPlatformData()[GroupIdx];

		//int32 NumCurves = GroupInfo.NumCurves;
		//int32 NumGuides = GroupInfo.NumGuides;

		FHairStrandsDatas StrandsData;
		FHairStrandsDatas GuidesData;
		Asset->GetHairStrandsDatas(GroupIdx, StrandsData, GuidesData);
		//const FHairStrandsDatas& GroupStrandData = (bUseGuides) ? GroupData.HairSimulationData : GroupData.HairRenderData;
		const FHairStrandsDatas& GroupStrandData = (bUseGuides) ? GuidesData : StrandsData;

		const FHairStrandsPoints& GroupStrandPoints = GroupStrandData.StrandsPoints;
		const TArray<FVector3f>& Positions = GroupStrandPoints.PointsPosition;
		const TArray<float>& Radii = GroupStrandPoints.PointsRadius;

		const FHairStrandsCurves& GroupStrandCurves = GroupStrandData.StrandsCurves;
		const TArray<uint16>& CurvesCounts = GroupStrandCurves.CurvesCount;
		const TArray<uint32>& CurvesOffsets = GroupStrandCurves.CurvesOffset;

		int32 NumCurves = CurvesCounts.Num();
		for (int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
		{
			int32 Count = (int32)CurvesCounts[CurveIndex];
			int32 Offset = (int32)CurvesOffsets[CurveIndex];

			TArrayView<FVector3f> CurvePositions = TArrayView<FVector3f>( (FVector3f*)&Positions[Offset], Count );
			TArrayView<float> CurveRadii = TArrayView<float>((float*)&Radii[Offset], Count);

			HairCurveFunc(CurvePositions, CurveRadii);

			//for (int32 k = 0; k < Count - 1; ++k)
			//{
			//	FSkeletalImplicitLine3d Line = { FSegment3d(FVector3d(Positions[Offset + k]), FVector3d(Positions[Offset + k + 1])), Radii[Offset + k] };
			//	//Line.Radius = 5.0;
			//	Lines.Add(Line);
			//}
		}

		//FString LinesName = FString::Printf(TEXT("STRANDGROUP_%d"), GroupIdx);
		//PreviewGeom->CreateOrUpdateLineSet(LinesName, GroupStrandCurves.Num(),
		//	[&](int32 CurveIndex, TArray<FRenderableLine>& LinesOut)
		//{
		//	int32 Count = (int32)CurvesCounts[CurveIndex];
		//	int32 Offset = (int32)CurvesOffsets[CurveIndex];
		//	for (int32 k = 0; k < Count - 1; ++k)
		//	{
		//		LinesOut.Add(FRenderableLine(Positions[Offset + k], Positions[Offset + k + 1], FColor::Red, 0.1f));
		//	}
		//});
	}

}





TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> UGroomToMeshTool::UpdateVoxelization()
{
	FVoxelizeSettings NewSettings;
	NewSettings.VoxelCount = Settings->VoxelCount;
	NewSettings.BlendPower = Settings->BlendPower;
	NewSettings.RadiusScale = Settings->RadiusScale;
	if (CachedVoxelizeSettings == NewSettings)
	{
		return CurrentVoxelizeResult;
	}

	check(TargetGroom.IsValid());
	AGroomActor* GroomActor = TargetGroom.Get();

	TUniquePtr<FDynamicPointSet3d> Points = MakeUnique<FDynamicPointSet3d>();
	TUniquePtr<FHairPointSetAttributes> PointAttribs = MakeUnique<FHairPointSetAttributes>();
	PointAttribs->Initialize(Points.Get());
	Points->SetExternallyManagedAttributes(PointAttribs.Get());

	FAxisAlignedBox3d Bounds = FAxisAlignedBox3d::Empty();
	ProcessHairCurvePoints(GroomActor, true, [&](const TArray<FVector3f>& Positions, const TArray<float>& Radii)
	{
		int32 NumPoints = Positions.Num();
		for (int32 k = 0; k < NumPoints; ++k)
		{
			FVector3d Position(Positions[k]);
			Bounds.Contain(Position);
			int32 NewVID = Points->AppendVertex(Position);
			PointAttribs->Radius->SetNewValue(NewVID, &Radii[k]);
		}
	});


	//TDynamicVerticesOctree3<FDynamicPointSet3d> PointOctree;
	//PointOctree.Initialize(Points.Get(), true);


	double BlendPower = NewSettings.BlendPower;
	int32 TargetVoxelCount = NewSettings.VoxelCount;
	double MaxRadius = 0.0;
	double RadiusScale = NewSettings.RadiusScale;

	TArray<FSkeletalImplicitLine3d> Lines;

	ProcessHairCurves(GroomActor, true, [&](const TArrayView<FVector3f>& Positions, const TArrayView<float>& Radii)
	{
		int32 Count = Positions.Num() - 1;
		for (int32 k = 0; k < Count; ++k)
		{
			FSkeletalImplicitLine3d Line;
			Line.Segment = FSegment3d(FVector3d(Positions[k]), FVector3d(Positions[k + 1]));
			Line.SetScaleFromRadius(RadiusScale * Radii[k]);
			Lines.Add(Line);
		}
	});

	TSkeletalRicciNaryBlend3<FSkeletalImplicitLine3d, double> Blend;
	Blend.BlendPower = BlendPower;
	Blend.Children.Reserve(Lines.Num());
	for (int32 k = 0; k < Lines.Num(); k++)
	{
		Blend.Children.Add(&Lines[k]);
		MaxRadius = FMathd::Max(Lines[k].GetRadius(), MaxRadius);
	}

	FAxisAlignedBox3d UseBounds = Blend.Bounds();
	FVector3d BoxDims = UseBounds.Diagonal();
	double MaxDimension = UseBounds.MaxDim();

	//MaxRadius = 10.0;		// why is this necessary??!?
	double MeshCellSize = (UseBounds.MaxDim() + 2.0 * MaxRadius) / double(TargetVoxelCount);
	//double MeshCellSize = UseBounds.MaxDim() / double(TargetVoxelCount);

	FMarchingCubes MarchingCubes;
	//MarchingCubes.CancelF = CancelF;

	MarchingCubes.CubeSize = MeshCellSize;
	MarchingCubes.IsoValue = 0.5f;
	MarchingCubes.Bounds = UseBounds;
	MarchingCubes.Bounds.Expand(2.0 * MaxRadius);
	//MarchingCubes.Bounds.Expand(32.0 * MeshCellSize);
	MarchingCubes.RootMode = ERootfindingModes::LerpSteps;
	//MarchingCubes.RootMode = ERootfindingModes::Bisection;
	MarchingCubes.RootModeSteps = 4;

	TArray<FVector3d> Seeds;
	for (int32 k = 0; k < Lines.Num(); ++k)
	{
		Seeds.Add( Lines[k].Segment.EndPoint() + Lines[k].GetRadius()*Lines[k].Segment.Direction );
		Seeds.Add( Lines[k].Segment.StartPoint() - Lines[k].GetRadius()*Lines[k].Segment.Direction );
	}

	MarchingCubes.Implicit = [&Blend](const FVector3d& Pt) { return Blend.Value(Pt); };

	MarchingCubes.GenerateContinuation(Seeds);

	// if we found zero triangles, try again w/ full grid search
	if (MarchingCubes.Triangles.Num() == 0)
	{
		MarchingCubes.Generate();
	}

	// clear implicit function...
	MarchingCubes.Implicit = nullptr;

	CurrentVoxelizeResult = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(&MarchingCubes);
	CachedVoxelizeSettings = NewSettings;
	return CurrentVoxelizeResult;
}




TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> UGroomToMeshTool::UpdateMorphology(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh)
{
	FMorphologySettings NewSettings;
	NewSettings.InputMesh = InputMesh;
	NewSettings.VoxelCount = Settings->VoxelCount;
	NewSettings.OpenDist = Settings->OpeningDist;
	NewSettings.CloseDist = Settings->ClosingDist;
	if (CachedMorphologySettings == NewSettings)
	{
		return CachedMorphologyResult;
	}

	CachedMorphologyResult = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(*InputMesh);

	if (NewSettings.CloseDist > 0 && CachedMorphologyResult->TriangleCount() > 0)
	{
		TImplicitMorphology<FDynamicMesh3> ImplicitMorphology;
		ImplicitMorphology.MorphologyOp = TImplicitMorphology<FDynamicMesh3>::EMorphologyOp::Close;
		ImplicitMorphology.Source = CachedMorphologyResult.Get();
		FDynamicMeshAABBTree3 Spatial(ImplicitMorphology.Source, true);
		ImplicitMorphology.SourceSpatial = &Spatial;
		ImplicitMorphology.SetCellSizesAndDistance(CachedMorphologyResult->GetBounds(true), NewSettings.CloseDist, NewSettings.VoxelCount, NewSettings.VoxelCount);

		CachedMorphologyResult = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(&ImplicitMorphology.Generate());
		if (CachedMorphologyResult->TriangleCount() == 0)
		{
			CachedPostprocessResult = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(*InputMesh);
		}
	}


	if (NewSettings.OpenDist > 0 && CachedMorphologyResult->TriangleCount() > 0)
	{
		TImplicitMorphology<FDynamicMesh3> ImplicitMorphology;
		ImplicitMorphology.MorphologyOp = TImplicitMorphology<FDynamicMesh3>::EMorphologyOp::Open;
		ImplicitMorphology.Source = CachedMorphologyResult.Get();
		FDynamicMeshAABBTree3 Spatial(ImplicitMorphology.Source, true);
		ImplicitMorphology.SourceSpatial = &Spatial;
		ImplicitMorphology.SetCellSizesAndDistance(CachedMorphologyResult->GetBounds(true), NewSettings.OpenDist, NewSettings.VoxelCount, NewSettings.VoxelCount);

		CachedMorphologyResult = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(&ImplicitMorphology.Generate());
		if (CachedMorphologyResult->TriangleCount() == 0)
		{
			CachedPostprocessResult = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(*InputMesh);
		}
	}

	CachedMorphologySettings = NewSettings;
	return CachedMorphologyResult;
}





enum class ETargetSpaceType
{
	Actor,
	Component,
	Frame,
	Transform,
	World
};


struct FTargetSpace
{
	ETargetSpaceType TargetSpace = ETargetSpaceType::Transform;

	AActor* TargetActor = nullptr;
	USceneComponent* TargetComponent = nullptr;
	FTransformSRT3d TargetTransform = FTransformSRT3d::Identity();
	FFrame3d TargetFrame = FFrame3d();

	FTargetSpace() {}

	FTargetSpace(AActor* TargetActorIn)
	{
		TargetSpace = ETargetSpaceType::Actor;
		TargetActor = TargetActorIn;
	}

	FTargetSpace(USceneComponent* TargetComponentIn)
	{
		TargetSpace = ETargetSpaceType::Component;
		TargetComponent = TargetComponentIn;
	}

	FTargetSpace(const FFrame3d& FrameIn)
	{
		TargetSpace = ETargetSpaceType::Frame;
		TargetFrame = FrameIn;
	}

	FTargetSpace(const FTransformSRT3d& TransformIn)
	{
		TargetSpace = ETargetSpaceType::Transform;
		TargetTransform = TransformIn;
	}

	static FTargetSpace World()
	{
		FTargetSpace Space;
		Space.TargetSpace = ETargetSpaceType::World;
		return Space;
	}
};




static void TransformToSpace(FDynamicMesh3* Mesh, UPrimitiveComponent* SourceComponent, const FTargetSpace& TargetSpace)
{
	// TODO: in certain cases we could combine these transforms, ie if there is no scaling, to avoid multiple MeshTransforms:: calls.
	// Could also add a variant there that applies multiple transforms in sequence to avoid write overhead

	// transform up to world
	FTransformSRT3d ToWorld(SourceComponent->GetComponentTransform());
	MeshTransforms::ApplyTransform(*Mesh, ToWorld);

	if (TargetSpace.TargetSpace == ETargetSpaceType::Actor)
	{
		check(TargetSpace.TargetActor);
		FTransformSRT3d ActorTransform(TargetSpace.TargetActor->GetActorTransform());
		MeshTransforms::ApplyTransformInverse(*Mesh, ActorTransform);

	}
	else if (TargetSpace.TargetSpace == ETargetSpaceType::Component)
	{
		check(TargetSpace.TargetComponent);
		FTransformSRT3d ComponentTransform(TargetSpace.TargetComponent->GetComponentTransform());
		MeshTransforms::ApplyTransformInverse(*Mesh, ComponentTransform);
	}
	else if (TargetSpace.TargetSpace == ETargetSpaceType::Frame)
	{
		MeshTransforms::WorldToFrameCoords(*Mesh, TargetSpace.TargetFrame);
	}
	else if (TargetSpace.TargetSpace == ETargetSpaceType::Transform)
	{
		MeshTransforms::ApplyTransform(*Mesh, TargetSpace.TargetTransform);
	}
	else if (TargetSpace.TargetSpace == ETargetSpaceType::World)
	{
		// we are good
	}
	else
	{
		check(false);		// how did we get here??
	}
}




static bool ExtractStaticMesh(UPrimitiveComponent* Component, int32 LODIndex, const FTargetSpace& TargetSpace, FDynamicMesh3& OutputMesh, bool bWantAttributes = true)
{
	if (ensure(Component) == false) return false;

	UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component);
	if (StaticMeshComponent)
	{
		UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
		if (ensure(StaticMesh))
		{
			const FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(LODIndex);
			if (ensure(MeshDescription))
			{
				FMeshDescriptionToDynamicMesh Converter;
				Converter.bCalculateMaps = false;
				Converter.bDisableAttributes = (bWantAttributes == false);
				Converter.bEnableOutputGroups = bWantAttributes;

				OutputMesh.Clear();
				Converter.Convert(MeshDescription, OutputMesh);

				TransformToSpace(&OutputMesh, StaticMeshComponent, TargetSpace);

				return true;
			}
		}
		return false;		// something failed
	}

	ensure(false);
	return false;		// unsupported type
}



static bool ExtractStaticMesh(AActor* Actor, int32 LODIndex, const FTargetSpace& TargetSpace, FDynamicMesh3& OutputMesh, bool bWantAttributes = true)
{
	if (ensure(Actor) == false) return false;

	AStaticMeshActor* StaticMeshActor = Cast<AStaticMeshActor>(Actor);
	if (StaticMeshActor)
	{
		return ExtractStaticMesh(StaticMeshActor->GetStaticMeshComponent(), LODIndex, TargetSpace, OutputMesh, bWantAttributes);
	}

	return false;
}






class FLoopProcessor
{
public:
	FDynamicMesh3* Mesh = nullptr;
	FEdgeLoop InitialLoop;

	FEdgeLoop CurrentLoop;
	TArray<FVector3d> PassBuffer;

	bool bApplySmoothingPerPass = true;
	double SmoothingAlpha = 0.2;

	FLoopProcessor()
	{
	}

	FLoopProcessor(FDynamicMesh3* MeshIn, const FEdgeLoop& LoopIn)
	{
		Initialize(MeshIn, LoopIn);
	}

	void Initialize(FDynamicMesh3* MeshIn, const FEdgeLoop& LoopIn)
	{
		Mesh = MeshIn;
		InitialLoop = LoopIn;
		CurrentLoop = InitialLoop;
	}


	void ApplyPasses(int32 NumPasses, TFunctionRef<FVector3d(int32, const FVector3d&)> UpdatePositionFunc)
	{
		for (int32 k = 0; k < NumPasses; ++k)
		{
			ApplyPass(UpdatePositionFunc);
		}
	}


	void ApplyPass( TFunctionRef<FVector3d(int32, const FVector3d&)> UpdatePositionFunc )
	{
		int32 NumV = CurrentLoop.GetVertexCount();
		PassBuffer.SetNumUninitialized(NumV);

		for (int32 k = 0; k < NumV; ++k)
		{
			int32 NextK = (k + 1) % NumV;
			int32 PrevK = (k == 0) ? (NumV-1) : (k-1);

			int32 vid = CurrentLoop.Vertices[k];
			FVector3d Pos = Mesh->GetVertex(vid);
			FVector3d NextPos = Mesh->GetVertex(CurrentLoop.Vertices[NextK]);
			FVector3d PrevPos = Mesh->GetVertex(CurrentLoop.Vertices[PrevK]);

			if (bApplySmoothingPerPass)
			{
				FVector3d Centroid = (NextPos + PrevPos) * 0.5;
				Pos = UE::Geometry::Lerp(Pos, Centroid, SmoothingAlpha);
			}

			Pos = UpdatePositionFunc(vid, Pos);

			PassBuffer[k] = Pos;
		}

		for (int32 k = 0; k < NumV; ++k)
		{
			FVector3d NewPos = PassBuffer[k];
			int32 vid = CurrentLoop.Vertices[k];
			Mesh->SetVertex(vid, NewPos);
		}
	}



};




TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> UGroomToMeshTool::UpdateClipMesh(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh)
{
	FClipMeshSettings NewSettings;
	NewSettings.InputMesh = InputMesh;
	NewSettings.ClipSource = Settings->ClipMeshActor.Get();
	if (CachedClipMeshSettings == NewSettings)
	{
		return CachedClipMeshResult;
	}

	CachedClipMeshResult = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(*InputMesh);
	FDynamicMesh3* EditMesh = CachedClipMeshResult.Get();

	// do a quick tiny-edges collapse pass
	double AvgEdgeLen = TMeshQueries<FDynamicMesh3>::AverageEdgeLength(*EditMesh);
	FQEMSimplification Simplifier(EditMesh);
	Simplifier.SimplifyToEdgeLength(AvgEdgeLen * 0.75);
	

	if (NewSettings.ClipSource != nullptr)
	{
		FDynamicMesh3 ClipMesh;
		if (ExtractStaticMesh(NewSettings.ClipSource, 0, FTargetSpace(TargetGroom.Get()), ClipMesh, false))
		{
			FDynamicMeshAABBTree3 BVTree(&ClipMesh, true);
			TFastWindingTree<FDynamicMesh3> WindingTree(&BVTree);

			FMeshVertexSelection SelectedVerts(EditMesh);
			SelectedVerts.SelectByPosition([&](const FVector3d& Position) { return WindingTree.IsInside(Position); });

			FMeshFaceSelection SelectedFaces(EditMesh, SelectedVerts, 1);
			//SelectedFaces.ContractBorderByOneRingNeighbours();
			SelectedFaces.LocalOptimize(true, true);

			FDynamicMeshEditor Editor(EditMesh);
			Editor.RemoveTriangles(SelectedFaces.AsArray(), true);

			FMeshBoundaryLoops BoundaryLoops(EditMesh, true);
			for (FEdgeLoop& Loop : BoundaryLoops.Loops)
			{
				double ProjectionSpeed = 0.3;
				double TuneSpeed = 0.9;
				double SmoothSpeed = 0.25;

				FLoopProcessor Processor(EditMesh, Loop);
				Processor.bApplySmoothingPerPass = true;
				Processor.SmoothingAlpha = SmoothSpeed;
				Processor.ApplyPasses(10, [&](int32 vid, const FVector3d& Pos)
				{
					FVector3d NearestPos = BVTree.FindNearestPoint(EditMesh->GetVertex(vid));
					return UE::Geometry::Lerp(Pos, NearestPos, ProjectionSpeed);
				});

				Processor.ApplyPasses(10, [&](int32 vid, const FVector3d& Pos)
				{
					FVector3d NearestPos = BVTree.FindNearestPoint(EditMesh->GetVertex(vid));
					return UE::Geometry::Lerp(Pos, NearestPos, TuneSpeed);
				});
			}



		}
	}

	CachedClipMeshResult->CompactInPlace();
	CachedClipMeshSettings = NewSettings;
	return CachedClipMeshResult;
}



TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> UGroomToMeshTool::UpdateSmoothing(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh)
{
	FSmoothingSettings NewSettings;
	NewSettings.InputMesh = InputMesh;
	NewSettings.Smoothness = Settings->Smoothness;
	NewSettings.VolumeCorrection = Settings->VolumeCorrection;
	if (CachedSmoothSettings == NewSettings)
	{
		return CachedSmoothResult;
	}

	CachedSmoothResult = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(*InputMesh);

	if (NewSettings.Smoothness > 0)
	{
		TSharedPtr<FMeshNormals> VtxNormals = MakeShared<FMeshNormals>(InputMesh.Get());
		if (NewSettings.VolumeCorrection != 0)
		{
			VtxNormals->ComputeVertexNormals();
		}

		FDynamicMesh3* EditMesh = CachedSmoothResult.Get();

		// normalize mesh area/position
		// compute area of the input mesh and compute normalization scaling factor
		FVector2d VolArea = TMeshQueries<FDynamicMesh3>::GetVolumeArea(*EditMesh);
		double UnitScalingMeasure = FMathd::Max(0.01, FMathd::Sqrt(VolArea.Y / 6.0));  // 6.0 is a bit arbitrary here...surface area of unit box
		FAxisAlignedBox3d Bounds = InputMesh->GetBounds(true);
		FVector3d SrcTranslate = Bounds.Center();
		MeshTransforms::Translate(*EditMesh, -SrcTranslate);
		double SrcScale = UnitScalingMeasure;
		MeshTransforms::Scale(*EditMesh, (1.0 / SrcScale) * FVector3d::One(), FVector3d::Zero());

		FSmoothingOpBase::FOptions Options;
		Options.BaseNormals = VtxNormals;
		Options.SmoothAlpha = NewSettings.Smoothness;
		Options.BoundarySmoothAlpha = 0.0;
		double NonlinearT = FMathd::Pow(NewSettings.Smoothness, 2.0);
		// this is an empirically-determined hack that seems to work OK to normalize the smoothing result for variable vertex count...
		double ScaledPower = (NonlinearT / 50.0) * EditMesh->VertexCount();
		Options.SmoothPower = ScaledPower;
		Options.bUniform = false;
		Options.bUseImplicit = true;
		Options.NormalOffset = NewSettings.VolumeCorrection / 10.0;		// rescale hack

		// weightmap
		//Options.bUseWeightMap = true;
		//Options.WeightMap = GetActiveWeightMap();
		//Options.WeightMapMinMultiplier = WeightMapProperties->MinSmoothMultiplier;

		TUniquePtr<FCotanSmoothingOp> MeshOp = MakeUnique<FCotanSmoothingOp>(EditMesh, Options);
		MeshOp->CalculateResult(nullptr);

		TUniquePtr<FDynamicMesh3> NewMesh = MeshOp->ExtractResult();

		// invert normalization transform
		MeshTransforms::Scale(*NewMesh, FVector3d(SrcScale, SrcScale, SrcScale), FVector3d::Zero());
		MeshTransforms::Translate(*NewMesh, SrcTranslate);

		CachedSmoothResult = TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe>(NewMesh.Release());
	}


	CachedSmoothSettings = NewSettings;
	return CachedSmoothResult;
}





TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> UGroomToMeshTool::UpdateSimplification(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh)
{
	FSimplifySettings NewSettings;
	NewSettings.InputMesh = InputMesh;
	NewSettings.TargetCount = Settings->VertexCount;
	if (CachedSimplifySettings == NewSettings)
	{
		return CachedSimplifyResult;
	}

	CachedSimplifyResult = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(*InputMesh);

	FQEMSimplification Simplifier(CachedSimplifyResult.Get());
	Simplifier.SimplifyToVertexCount(NewSettings.TargetCount);
	CachedSimplifyResult->CompactInPlace();

	CachedSimplifySettings = NewSettings;
	return CachedSimplifyResult;
}



TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> UGroomToMeshTool::UpdatePostprocessing(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh)
{
	FPostprocessSettings NewSettings;
	NewSettings.InputMesh = InputMesh;
	NewSettings.UVGenMode = Settings->UVMode;
	if (CachedPostprocessSettings == NewSettings)
	{
		return CachedPostprocessResult;
	}

	CachedPostprocessResult = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(*InputMesh);

	FMeshNormals::InitializeMeshToPerTriangleNormals(CachedPostprocessResult.Get());

	CachedPostprocessResult = UpdateUVs(CachedPostprocessResult, NewSettings.UVGenMode);

	CachedPostprocessSettings = NewSettings;
	return CachedPostprocessResult;
}




template<typename EnumerableType>
void PartitionIndices(EnumerableType Enumerable, 
	TFunctionRef<FVector3d(int32)> GetPositionForIndexFunc,
	FLine3d Line, FInterval1d CenterRange,
	TArray<int32>& BelowOut, TArray<int32>& CenterOut, TArray<int32>& AboveOut)
{
	for (int32 vid : Enumerable)
	{
		FVector3d Position = GetPositionForIndexFunc(vid);
		double T = Line.Project(Position);
		if (CenterRange.Contains(T))
		{
			CenterOut.Add(vid);
		}
		else if (T < CenterRange.Min)
		{
			BelowOut.Add(vid);
		}
		else
		{
			AboveOut.Add(vid);
		}
	}
}



TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> UGroomToMeshTool::UpdateUVs(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh, EGroomToMeshUVMode UVMode)
{
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> Result;

	if (InputMesh->IsClosed())
	{
		Result = UpdateUVs_ExpMapPlaneSplits(InputMesh, true);
		return Result;
	}

	switch (UVMode)
	{
	default:
	case EGroomToMeshUVMode::PlanarSplitting:
		Result = UpdateUVs_ExpMapPlaneSplits(InputMesh, false);
		break;
	case EGroomToMeshUVMode::MinimalConformal:
		Result = UpdateUVs_MinimalConformal(InputMesh);
		break;
	case EGroomToMeshUVMode::PlanarSplitConformal:
		Result = UpdateUVs_ExpMapPlaneSplits(InputMesh, true);
		break;
	}

	// check that all triangles have valid UVs, because otherwise FUVLayoutOp will crash
	bool bAllUVsValid = true;
	auto AttribSet = Result->Attributes()->PrimaryUV();
	for (int32 TriangleID : Result->TriangleIndicesItr())
	{
		bAllUVsValid = bAllUVsValid && AttribSet->IsSetTriangle(TriangleID);
	}
	if (!bAllUVsValid)
	{
		return Result;
	}

	// run UV layout
	FUVLayoutOp UVLayout;
	UVLayout.OriginalMesh = Result;
	UVLayout.UVLayoutMode = EUVLayoutOpLayoutModes::RepackToUnitRect;
	UVLayout.TextureResolution = 64;
	UVLayout.UVScaleFactor = 1.0;
	UVLayout.CalculateResult(nullptr);
	*Result = MoveTemp(*UVLayout.ExtractResult());

	return Result;
}



static void GenerateUVTiling(int32 NumTiles, TArray<FAxisAlignedBox2d>& BoxesOut, double UVSpaceGutterWidth)
{
	if (NumTiles == 1)
	{
		FAxisAlignedBox2d FullBox(FVector2d::Zero(), FVector2d::One());
		FullBox.Expand(-UVSpaceGutterWidth);
		BoxesOut.Add(FullBox);
		return;
	}

	int32 Columns = (int)FMathd::Ceil(FMathd::Sqrt((double)NumTiles));
	int32 Rows = Columns;

	double W = 1.0 / (double)Columns;

	for (int32 ri = 0; ri < Rows; ++ri)
	{
		double y = (double)ri * W;
		for (int32 ci = 0; ci < Columns; ++ci)
		{
			double x = (double)ci * W;

			FAxisAlignedBox2d Box(FVector2d(x, y), FVector2d(x + W, y + W));
			Box.Expand(-UVSpaceGutterWidth);
			BoxesOut.Add(Box);
			if (BoxesOut.Num() == NumTiles)
			{
				return;
			}
		}
	}
}



TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> UGroomToMeshTool::UpdateUVs_MinimalConformal(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh)
{
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> Result = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(*InputMesh);
	FDynamicMesh3* EditMesh = Result.Get();

	FDynamicMeshUVEditor UVEditor(EditMesh, 0, true);

	FMeshConnectedComponents MeshComponents(EditMesh);
	MeshComponents.FindConnectedTriangles();
	TArray<FAxisAlignedBox2d> TilingBoxes;
	GenerateUVTiling(MeshComponents.Num(), TilingBoxes, 0.01);
	for (int32 k = 0; k < MeshComponents.Num(); ++k)
	{
		FMeshConnectedComponents::FComponent& Component = MeshComponents.GetComponent(k);
		FUVEditResult EditResult;
		check(Component.Indices.Num() > 1);
		if ( UVEditor.SetTriangleUVsFromFreeBoundaryConformal(Component.Indices, &EditResult) == false )
		{
			// if we failed at conformal, fallback to expmap for this island
			EditResult = FUVEditResult();
			if ( UVEditor.SetTriangleUVsFromExpMap(Component.Indices) == false )
			{
				// if we somehow failed at conformal, fallback to trivial planar projection
				UVEditor.SetTriangleUVsFromProjection(Component.Indices, FFrame3d());
			}
		}

		// destroys relative scale between islands (but for conformal it is not meaninful anyway...)
		//UE::MeshUVTransforms::FitToBox(UVEditor.GetOverlay(), EditResult.NewUVElements, TilingBoxes[k], true);
	}

	return Result;
}




TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> UGroomToMeshTool::UpdateUVs_ExpMapPlaneSplits(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh, bool bRecalcAsConformal)
{
	int32 SplitAxisIndex1 = 0;
	int32 SplitAxisIndex2 = 1;
	double MiddleFraction1 = 0.6;
	double MiddleFraction2 = 0.6;

	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> Result = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(*InputMesh);
	FDynamicMesh3* EditMesh = Result.Get();
	
	FDynamicMeshAABBTree3 BVTree(EditMesh, true);

	FAxisAlignedBox3d Bounds = EditMesh->GetBounds(true);
	FVector3d Center = Bounds.Center();
	FVector3d Extents = Bounds.Extents();

	// split left/right/middle

	FVector3d SplitAxis1 = FVector3d::Zero();
	SplitAxis1[SplitAxisIndex1] = 1.0;
	FInterval1d AxisInterval1(-Extents[SplitAxisIndex1]*MiddleFraction1, Extents[SplitAxisIndex1]*MiddleFraction1);

	TArray<int32> LeftT, RightT, MiddleT;
	PartitionIndices(EditMesh->TriangleIndicesItr(), [&](int32 tid) { return EditMesh->GetTriCentroid(tid); },
		FLine3d(Center, SplitAxis1), AxisInterval1, LeftT, MiddleT, RightT);

	int32 LeftGID = EditMesh->AllocateTriangleGroup();
	int32 RightGID = EditMesh->AllocateTriangleGroup();

	FMeshFaceSelection MiddleFaces(EditMesh);
	MiddleFaces.Select(MiddleT);
	MiddleFaces.LocalOptimize(true, true);

	FMeshFaceSelection LeftFaces(EditMesh);
	LeftFaces.Select(LeftT);
	LeftFaces.ExpandToOneRingNeighbours();
	LeftFaces.ExpandToOneRingNeighbours();
	LeftFaces.Deselect(MiddleFaces);
	for (int32 tid : LeftFaces)
	{
		EditMesh->SetTriangleGroup(tid, LeftGID);
	}

	FMeshFaceSelection RightFaces(EditMesh);
	RightFaces.Select(RightT);
	RightFaces.ExpandToOneRingNeighbours();
	RightFaces.ExpandToOneRingNeighbours();
	RightFaces.Deselect(MiddleFaces);
	for (int32 tid : RightFaces)
	{
		EditMesh->SetTriangleGroup(tid, RightGID);
	}


	// split middle group again
	
	FVector3d SplitAxis2 = FVector3d::Zero();
	SplitAxis2[SplitAxisIndex2] = 1.0;
	FInterval1d AxisInterval2(-Extents[SplitAxisIndex2] * MiddleFraction2, Extents[SplitAxisIndex2] * MiddleFraction2);

	MiddleT.Reset();
	TArray<int32> FrontT, BackT;
	PartitionIndices(MiddleFaces, [&](int32 tid) { return EditMesh->GetTriCentroid(tid); },
		FLine3d(Center, SplitAxis2), AxisInterval2, FrontT, MiddleT, BackT);

	int32 FrontGID = EditMesh->AllocateTriangleGroup();
	int32 MiddleGID = EditMesh->AllocateTriangleGroup();
	int32 BackGID = EditMesh->AllocateTriangleGroup();

	MiddleFaces = FMeshFaceSelection(EditMesh);
	MiddleFaces.Select(MiddleT);
	MiddleFaces.LocalOptimize(true, true);
	for (int32 tid : MiddleFaces)
	{
		EditMesh->SetTriangleGroup(tid, MiddleGID);
	}

	FMeshFaceSelection FrontFaces(EditMesh);
	FrontFaces.Select(FrontT);
	FrontFaces.ExpandToOneRingNeighbours();
	FrontFaces.ExpandToOneRingNeighbours();
	FrontFaces.Deselect(MiddleFaces);
	for (int32 tid : FrontFaces)
	{
		EditMesh->SetTriangleGroup(tid, FrontGID);
	}

	FMeshFaceSelection BackFaces(EditMesh);
	BackFaces.Select(BackT);
	BackFaces.ExpandToOneRingNeighbours();
	BackFaces.ExpandToOneRingNeighbours();
	BackFaces.Deselect(MiddleFaces);
	for (int32 tid : BackFaces)
	{
		EditMesh->SetTriangleGroup(tid, BackGID);
	}


	// generate UVs for all UV islands

	FDynamicMeshUVEditor UVEditor(EditMesh, 0, true);

	FMeshConnectedComponents GroupComponents(EditMesh);
	GroupComponents.FindConnectedTriangles([EditMesh](int32 Triangle0, int32 Triangle1) {
		return EditMesh->GetTriangleGroup(Triangle0) == EditMesh->GetTriangleGroup(Triangle1);
	});

	TArray<FAxisAlignedBox2d> TilingBoxes;
	GenerateUVTiling(GroupComponents.Num(), TilingBoxes, 0.01);
	for ( int32 k = 0; k < GroupComponents.Num(); ++k )
	{
		FMeshConnectedComponents::FComponent& Component = GroupComponents.GetComponent(k);
		if (bRecalcAsConformal && Component.Indices.Num() > 1)
		{
			if (UVEditor.SetTriangleUVsFromFreeBoundaryConformal(Component.Indices) == false)
			{
				// if we failed at conformal, fallback to expmap for this island
				if (UVEditor.SetTriangleUVsFromExpMap(Component.Indices) == false)
				{
					// if we somehow failed at conformal, fallback to trivial planar projection
					UVEditor.SetTriangleUVsFromProjection(Component.Indices, FFrame3d());
				}
			}
		}
		else
		{
			UVEditor.SetTriangleUVsFromExpMap(Component.Indices);
		}
	}

	return Result;
}



void UGroomToMeshTool::UpdateLineSet()
{
	FString GuideCurvesName = TEXT("HAIRSTRANDS_CURVES");
	if (Settings->bShowGuides)
	{
		check(TargetGroom.IsValid());
		AGroomActor* GroomActor = TargetGroom.Get();
		PreviewGeom->CreateOrUpdateLineSet(GuideCurvesName, 1, [&](int32 CurveIndex, TArray<FRenderableLine>& LinesOut)
		{
			ProcessHairCurves(GroomActor, true, [&](const TArrayView<FVector3f>& Positions, const TArrayView<float>& Radii)
			{
				int32 Count = Positions.Num() - 1;
				for (int32 k = 0; k < Count; ++k)
				{
					LinesOut.Add(FRenderableLine((FVector)Positions[k], (FVector)Positions[k + 1], FColor::Red, 0.1f));
				}
			});
		});
	}
	else
	{
		PreviewGeom->RemoveLineSet(GuideCurvesName);
	}
}




void UGroomToMeshTool::UpdatePreview(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> ResultMesh)
{
	CurrentMesh = FDynamicMesh3(*ResultMesh);

	// compute normals
	if (CurrentMesh.TriangleCount() == 0)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("NoTrianglesWarning", "The Output Mesh does not contain any triangles with the current settings"), EToolMessageLevel::UserWarning);
		PreviewMesh->UpdatePreview(&CurrentMesh);
	}
	else
	{
		GetToolManager()->DisplayMessage({}, EToolMessageLevel::UserWarning);

		FMeshNormals::InitializeOverlayToPerVertexNormals(CurrentMesh.Attributes()->PrimaryNormals(), false);

		if (Settings->bShowSideBySide)
		{
			FAxisAlignedBox3d Bounds = CurrentMesh.GetBounds();
			FDynamicMesh3 TmpMesh(CurrentMesh);
			MeshTransforms::Translate(TmpMesh, Bounds.Width() * FVector3d::UnitX());
			PreviewMesh->UpdatePreview(&TmpMesh);
		}
		else
		{
			PreviewMesh->UpdatePreview(&CurrentMesh);
		}
	}

	UpdateLineSet();
}



#undef LOCTEXT_NAMESPACE
