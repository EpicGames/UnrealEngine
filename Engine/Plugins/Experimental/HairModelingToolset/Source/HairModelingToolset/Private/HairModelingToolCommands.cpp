// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairModelingToolCommands.h"
#include "Styling/AppStyle.h"
#include "InputCoreTypes.h"
#include "HairModelingToolsStyle.h"

#define LOCTEXT_NAMESPACE "HairModelingToolCommands"



FHairModelingToolCommands::FHairModelingToolCommands() :
	TCommands<FHairModelingToolCommands>(
		"HairModelingToolCommands", // Context name for fast lookup
		NSLOCTEXT("Contexts", "HairModelingToolCommands", "Hair Modeling Tools"), // Localized context name for displaying
		NAME_None, // Parent
		FHairModelingToolsStyle::Get()->GetStyleSetName() // Icon Style Set
		)
{
}


void FHairModelingToolCommands::RegisterCommands()
{
	UI_COMMAND(HairToolsTabButton, "Groom", "Groom Modeling Toolset", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(BeginGroomToMeshTool, "Helmet", "Generate Helmet Mesh for Selected Groom", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginGenerateLODMeshesTool, "HlmLOD", "Generate LODS for Hair Helmet", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginGroomCardsEditorTool, "CardsEd", "Edit Hair Cards", EUserInterfaceActionType::ToggleButton, FInputChord());
}




#undef LOCTEXT_NAMESPACE
