// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairModelingToolsStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "Interfaces/IPluginManager.h"
#include "SlateOptMacros.h"
#include "Styling/SlateStyleMacros.h"

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FHairModelingToolsStyle::InContent( RelativePath, ".png" ), __VA_ARGS__ )
#define RootToContentDir StyleSet->RootToContentDir

FString FHairModelingToolsStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("HairModelingTools"))->GetContentDir();
	return (ContentDir / RelativePath) + Extension;
}

TSharedPtr< FSlateStyleSet > FHairModelingToolsStyle::StyleSet = nullptr;
TSharedPtr< class ISlateStyle > FHairModelingToolsStyle::Get() { return StyleSet; }

FName FHairModelingToolsStyle::GetStyleSetName()
{
	static FName ModelingToolsStyleName(TEXT("HairModelingToolsStyle"));
	return ModelingToolsStyleName;
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FHairModelingToolsStyle::Initialize()
{
	// Const icon sizes
	const FVector2D Icon20x20(20.0f, 20.0f);

	// Only register once
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet(GetStyleSetName()));
	StyleSet->SetContentRoot(IPluginManager::Get().FindPlugin(TEXT("HairModelingToolset"))->GetContentDir());
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	{
		StyleSet->Set("HairModelingToolCommands.HairToolsTabButton", new IMAGE_BRUSH_SVG("Icons/HairHelmet", Icon20x20));
		StyleSet->Set("HairModelingToolCommands.HairToolsTabButton.Small", new IMAGE_BRUSH_SVG("Icons/HairHelmet", Icon20x20));

		StyleSet->Set("HairModelingToolCommands.BeginGroomCardsEditorTool", new IMAGE_BRUSH_SVG("Icons/CardsEditor", Icon20x20));
		StyleSet->Set("HairModelingToolCommands.BeginGroomCardsEditorTool.Small", new IMAGE_BRUSH_SVG("Icons/CardsEditor", Icon20x20));
		StyleSet->Set("HairModelingToolCommands.BeginGenerateLODMeshesTool", new IMAGE_BRUSH_SVG("Icons/GenLODs", Icon20x20));
		StyleSet->Set("HairModelingToolCommands.BeginGenerateLODMeshesTool.Small", new IMAGE_BRUSH_SVG("Icons/GenLODs", Icon20x20));
		StyleSet->Set("HairModelingToolCommands.BeginGroomToMeshTool", new IMAGE_BRUSH_SVG("Icons/HairHelmet", Icon20x20));
		StyleSet->Set("HairModelingToolCommands.BeginGroomToMeshTool.Small", new IMAGE_BRUSH_SVG("Icons/HairHelmet", Icon20x20));

	}

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
};

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef IMAGE_PLUGIN_BRUSH
#undef RootToContentDir

void FHairModelingToolsStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}
