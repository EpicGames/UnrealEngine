// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairModelingToolsetModule.h"
#include "Features/IModularFeatures.h"
#include "HairModelingToolsStyle.h"
#include "HairModelingToolCommands.h"

#include "GroomToMeshTool.h"
#include "GroomCardsEditorTool.h"
#include "GenerateLODMeshesTool.h"

#define LOCTEXT_NAMESPACE "FHairModelingToolsetModule"


void FHairModelingToolsetModule::StartupModule()
{
	FHairModelingToolsStyle::Initialize();
	FHairModelingToolCommands::Register();

	IModularFeatures::Get().RegisterModularFeature(IModelingModeToolExtension::GetModularFeatureName(), this);
}

void FHairModelingToolsetModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(IModelingModeToolExtension::GetModularFeatureName(), this);

	FHairModelingToolCommands::Unregister();
	FHairModelingToolsStyle::Shutdown();
}


FText FHairModelingToolsetModule::GetExtensionName()
{
	return LOCTEXT("ExtensionName", "HairTools");
}

FText FHairModelingToolsetModule::GetToolSectionName()
{
	return LOCTEXT("SectionName", "Hair");
}

void FHairModelingToolsetModule::GetExtensionTools(const FExtensionToolQueryInfo& QueryInfo, TArray<FExtensionToolDescription>& ToolsOut)
{
	FExtensionToolDescription GroomToMeshToolInfo;
	GroomToMeshToolInfo.ToolName = LOCTEXT("HairGroomToMeshTool", "Groom To Mesh");
	GroomToMeshToolInfo.ToolCommand = FHairModelingToolCommands::Get().BeginGroomToMeshTool;
	GroomToMeshToolInfo.ToolBuilder = NewObject<UGroomToMeshToolBuilder>();
	ToolsOut.Add(GroomToMeshToolInfo);

	FExtensionToolDescription GenerateHairLODMeshesToolInfo;
	GenerateHairLODMeshesToolInfo.ToolName = LOCTEXT("HairLODMeshesTool", "Hair LODs");
	GenerateHairLODMeshesToolInfo.ToolCommand = FHairModelingToolCommands::Get().BeginGenerateLODMeshesTool;
	GenerateHairLODMeshesToolInfo.ToolBuilder = NewObject<UGenerateLODMeshesToolBuilder>();
	ToolsOut.Add(GenerateHairLODMeshesToolInfo);

	FExtensionToolDescription GroomCardsEditorToolInfo;
	GroomCardsEditorToolInfo.ToolName = LOCTEXT("GroomCardsEditorTool", "Cards Editor");
	GroomCardsEditorToolInfo.ToolCommand = FHairModelingToolCommands::Get().BeginGroomCardsEditorTool;
	GroomCardsEditorToolInfo.ToolBuilder = NewObject<UGroomCardsEditorToolBuilder>();
	ToolsOut.Add(GroomCardsEditorToolInfo);
}

bool FHairModelingToolsetModule::GetExtensionExtendedInfo(FModelingModeExtensionExtendedInfo& InfoOut)
{
	InfoOut.ExtensionCommand = FHairModelingToolCommands::Get().HairToolsTabButton;
	InfoOut.ToolPaletteButtonTooltip = LOCTEXT("HairExtensionTooltip", "Tools for working with Groom meshes and cards");
	return true;
}




#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FHairModelingToolsetModule, HairModelingToolset)
