// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "SingleSelectionTool.h"
#include "InteractiveToolBuilder.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "MeshOpPreviewHelpers.h"
#include "CleaningOps/SimplifyMeshOp.h"
#include "Properties/MeshStatisticsProperties.h"
#include "Properties/RemeshProperties.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "GenerateLODMeshesTool.generated.h"

/**
 *
 */
UCLASS()
class HAIRMODELINGTOOLSET_API UGenerateLODMeshesToolBuilder : public USingleSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()
public:
	virtual USingleSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
};



USTRUCT()
struct HAIRMODELINGTOOLSET_API FLODLevelGenerateSettings
{
	GENERATED_BODY()

	/** Simplification Scheme  */
	UPROPERTY(EditAnywhere, Category = Options)
	ESimplifyType SimplifierType = ESimplifyType::UEStandard;

	/** Simplification Target Type  */
	UPROPERTY(EditAnywhere, Category = Options)
	ESimplifyTargetType TargetMode = ESimplifyTargetType::Percentage;

	/** Target percentage */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "0", UIMax = "100", EditConditionHides, EditCondition = "TargetMode == ESimplifyTargetType::Percentage"))
	int32 TargetPercentage = 100;

	/** Target vertex/triangle count */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "4", UIMax = "10000", ClampMin = "1", ClampMax = "9999999999", EditConditionHides, EditCondition = "TargetMode == ESimplifyTargetType::TriangleCount || TargetMode == ESimplifyTargetType::VertexCount"))
	int32 TargetCount = 500;

	/** Target vertex/triangle count */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Options)
	bool bReproject = false;

	UPROPERTY(VisibleAnywhere, Category = Options)
	FString Result;

	bool operator!=(const FLODLevelGenerateSettings& Other)
	{
		return SimplifierType != Other.SimplifierType || TargetMode != Other.TargetMode || TargetPercentage != Other.TargetPercentage || TargetCount != Other.TargetCount || bReproject != Other.bReproject;
	}
};



/**
 * Standard properties of the Simplify operation
 */
UCLASS()
class HAIRMODELINGTOOLSET_API UGenerateLODMeshesToolProperties : public UMeshConstraintProperties
{
	GENERATED_BODY()
public:
	UGenerateLODMeshesToolProperties();

	/** Simplification Target Type  */
	//UPROPERTY(EditAnywhere, Category = Options)
	UPROPERTY()
	ESimplifyTargetType TargetMode = ESimplifyTargetType::Percentage;

	/** Simplification Scheme  */
	//UPROPERTY(EditAnywhere, Category = Options)
	UPROPERTY()
	ESimplifyType SimplifierType = ESimplifyType::UEStandard;

	/** Output LOD Assets will be numbered starting at this number */
	UPROPERTY(EditAnywhere, Category = Options)
	int NameIndexBase = 0;

	/** Target percentage of original triangle count */
	//UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "0", UIMax = "100", EditCondition = "TargetMode == ESimplifyTargetType::Percentage"))
	UPROPERTY()
	int TargetPercentage = 50;

	/** Target edge length */
	//UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "3.0", UIMax = "10.0", ClampMin = "0.001", ClampMax = "1000.0", EditCondition = "TargetMode == ESimplifyTargetType::EdgeLength && SimplifierType != ESimplifyType::UEStandard"))
	UPROPERTY()
	float TargetEdgeLength;

	/** Target triangle/vertex count */
	//UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "4", UIMax = "10000", ClampMin = "1", ClampMax = "9999999999", EditCondition = "TargetMode == ESimplifyTargetType::TriangleCount"))
	UPROPERTY()
	int TargetCount = 1000;

	/** Discard UVs and existing normals, allowing the simplifier to ignore any UV and normal seams. New per-vertex normals are computed. */
	//UPROPERTY(EditAnywhere, Category = Options)
	UPROPERTY()
	bool bDiscardAttributes = false;

	/** If true, display wireframe */
	UPROPERTY(EditAnywhere, Category = Display)
	bool bShowWireframe = false;

	/** Display colors corresponding to the mesh's polygon groups */
	//UPROPERTY(EditAnywhere, Category = Display)
	UPROPERTY()
	bool bShowGroupColors = false;

	/** Enable projection back to input mesh */
	//UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay)
	UPROPERTY()
	bool bReproject;

	UPROPERTY(EditAnywhere, Category = Options)
	TArray<FLODLevelGenerateSettings> LODLevels;

};



class FGenerateLODOperatorFactory : public UE::Geometry::IDynamicMeshOperatorFactory
{
public:
	TWeakObjectPtr<UGenerateLODMeshesTool> ParentTool;
	FLODLevelGenerateSettings LODSettings;
	FTransform UseTransform;

	// IDynamicMeshOperatorFactory API
	virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;
};



/**
 * Simple Mesh Simplifying Tool
 */
UCLASS()
class HAIRMODELINGTOOLSET_API UGenerateLODMeshesTool : public USingleSelectionMeshEditingTool
{
	GENERATED_BODY()

public:
	virtual void Setup() override;
	virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	virtual void OnTick(float DeltaTime) override;
	virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	virtual bool CanAccept() const override;



private:
	UPROPERTY()
	TObjectPtr<UGenerateLODMeshesToolProperties> SimplifyProperties;

	UPROPERTY()
	TArray<TObjectPtr<UMeshOpPreviewWithBackgroundCompute>> Previews;

	TArray<TUniquePtr<FGenerateLODOperatorFactory>> PreviewFactories;
	friend class FGenerateLODOperatorFactory;

	void UpdateNumPreviews();
	void InvalidateAllPreviews();

	TSharedPtr<FMeshDescription, ESPMode::ThreadSafe> OriginalMeshDescription;
	// Dynamic Mesh versions precomputed in Setup (rather than recomputed for every simplify op)
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3, ESPMode::ThreadSafe> OriginalMeshSpatial;

	TArray<FLODLevelGenerateSettings> CachedLODLevels;

	UE::Geometry::FAxisAlignedBox3d WorldBounds;

	void GenerateAssets();
	void UpdateVisualization();

	void OnPreviewUpdated(UMeshOpPreviewWithBackgroundCompute*);
};
