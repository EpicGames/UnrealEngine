// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/PimplPtr.h"
#include "BaseTools/MeshSurfacePointMeshEditingTool.h"
#include "InteractiveToolBuilder.h"
#include "PreviewMesh.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "Mechanics/SpaceCurveDeformationMechanic.h"
#include "Transforms/MultiTransformer.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "GroupTopology.h"

#include "GroomCardsEditorTool.generated.h"

class AGroomActor;
class AStaticMeshActor;

class FEditableGroomCardSet;
class FGroomCardEdit;
class FMeshVertexChangeBuilder;

class UGroomCardsEditorTool;

/**
 *
 */
UCLASS()
class HAIRMODELINGTOOLSET_API UGroomCardsEditorToolBuilder : public UMeshSurfacePointMeshEditingToolBuilder
{
	GENERATED_BODY()
public:
	virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
};




UENUM()
enum class EEditGroomCardsToolActions
{
	NoAction,
	Delete,
	SelectionClear,
	SelectionFill,
	SelectionAddNext,
	SelectionAddPrevious,
	SelectionAddToEnd,
	SelectionAddToStart
};


UCLASS()
class HAIRMODELINGTOOLSET_API UEditGroomCardsToolActionPropertySet : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<UGroomCardsEditorTool> ParentTool;

	void Initialize(UGroomCardsEditorTool* ParentToolIn) { ParentTool = ParentToolIn; }

	void PostAction(EEditGroomCardsToolActions Action);
};


UCLASS()
class HAIRMODELINGTOOLSET_API USelectGroomCardsToolActions : public UEditGroomCardsToolActionPropertySet
{
	GENERATED_BODY()
public:
	/** Clear the current selection */
	UFUNCTION(CallInEditor, Category = Selection, meta = (DisplayPriority = 1))
	void Clear() { PostAction(EEditGroomCardsToolActions::SelectionClear); }

	/** Select unselected points along curve between selected points */
	UFUNCTION(CallInEditor, Category = Selection, meta = (DisplayPriority = 1))
	void Fill() { PostAction(EEditGroomCardsToolActions::SelectionFill); }

	/** Add the next vertex along the card curve to the selection */
	UFUNCTION(CallInEditor, Category = Selection, meta = (DisplayPriority = 3))
	void AddNext() { PostAction(EEditGroomCardsToolActions::SelectionAddNext); }

	/** Add the previous vertex along the card curve to the selection */
	UFUNCTION(CallInEditor, Category = Selection, meta = (DisplayPriority = 4))
	void AddPrevious() { PostAction(EEditGroomCardsToolActions::SelectionAddPrevious); }

	/** Select all vertices to the end of the card */
	UFUNCTION(CallInEditor, Category = Selection, meta = (DisplayPriority = 5))
	void ToEnd() { PostAction(EEditGroomCardsToolActions::SelectionAddToEnd); }

	/** Select all vertices to the start of the card */
	UFUNCTION(CallInEditor, Category = Selection, meta = (DisplayPriority = 6))
	void ToStart() { PostAction(EEditGroomCardsToolActions::SelectionAddToStart); }

};



UCLASS()
class HAIRMODELINGTOOLSET_API UEditGroomCardsToolActions : public UEditGroomCardsToolActionPropertySet
{
	GENERATED_BODY()
public:
	/** Delete the current selected cards */
	UFUNCTION(CallInEditor, Category = FaceEdits, meta = (DisplayName = "Delete", DisplayPriority = 10))
	void Delete() { PostAction(EEditGroomCardsToolActions::Delete); }
};









UCLASS()
class HAIRMODELINGTOOLSET_API UGroomCardsInfoToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, Category = Statistics)
	int32 NumCards = 0;

	UPROPERTY(VisibleAnywhere, Category = Statistics)
	int32 NumVertices = 0;

	UPROPERTY(VisibleAnywhere, Category = Statistics)
	int32 NumTriangles = 0;

};



/**
 *
 */
UCLASS()
class HAIRMODELINGTOOLSET_API UGroomCardsEditorTool : public UMeshSurfacePointTool, public IClickBehaviorTarget
{
	GENERATED_BODY()

public:
	UGroomCardsEditorTool();

	virtual void SetWorld(UWorld* World) { this->TargetWorld = World; }

	virtual void Setup() override;
	virtual void Shutdown(EToolShutdownType ShutdownType) override;

	virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override;
	virtual bool CanAccept() const override;


	// UMeshSurfacePointTool API
	virtual bool HitTest(const FRay& Ray, FHitResult& OutHit) override;
	virtual void OnBeginDrag(const FRay& Ray) override;
	virtual void OnUpdateDrag(const FRay& Ray) override;
	virtual void OnEndDrag(const FRay& Ray) override;
	virtual void OnCancelDrag() override;
	virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	virtual void OnEndHover() override;

	// IClickDragBehaviorTarget API
	virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;

	// IClickBehaviorTarget API
	virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	virtual void OnClicked(const FInputDeviceRay& ClickPos) override;


	// actions API
	void RequestAction(EEditGroomCardsToolActions ActionType);

protected:
	UPROPERTY()
	TObjectPtr<USelectGroomCardsToolActions> SelectActions;

	UPROPERTY()
	TObjectPtr<UEditGroomCardsToolActions> EditActions;

	UPROPERTY()
	TObjectPtr<UGroomCardsInfoToolProperties> InfoProperties;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh;

	UPROPERTY()
	TSoftObjectPtr<AGroomActor> TargetGroom;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> PreviewGeom;

protected:		// materials

	UPROPERTY()
	TObjectPtr<UMaterialInterface> MeshMaterial;

	UPROPERTY()
	TObjectPtr<UMaterialInterface> UVMaterial;

protected:		// mechanics

	UPROPERTY()
	TObjectPtr<UPolygonSelectionMechanic> CardMeshSelectionMechanic;

	UPROPERTY()
	TObjectPtr<USpaceCurveDeformationMechanic> ControlPointsMechanic;


	bool bSelectionStateDirty = false;
	void OnSelectionModifiedEvent();


protected:
	UWorld* TargetWorld = nullptr;

	TPimplPtr<FEditableGroomCardSet> EditableCardSet;

	void BeginCardEdit(int32 CardGroupID);
	void RestoreCardEdit(const FGroomCardEdit* RestoreEdit);
	void EndCardEdit();
	void CompleteActiveCardEditAndEmitChanges();

	TPimplPtr<FGroomCardEdit> ActiveCardEdit;
	int32 ActiveCardGroupID = -1;
	bool bCurveUpdatePending = false;
	bool bActiveCardEditUpdated = false;
	void UpdateOnCurveEdit();

	FMeshVertexChangeBuilder* ActiveVertexChange = nullptr;
	void BeginMoveChange();
	void EndMoveChange();

	TSharedPtr<FSpaceCurveSource> CurveSourceAdapter;


	void UpdateLineSet();

	bool bVisualizationChanged = false;

	bool bSetupValid = false;
	void InitializeMesh();

	TUniquePtr<FGroupTopology> Topology;
	void RecomputeTopology();

	void OnPreviewMeshChanged();
	FDelegateHandle OnPreviewMeshChangedHandle;

protected:
	// operation support

	EEditGroomCardsToolActions PendingAction = EEditGroomCardsToolActions::NoAction;

	int32 CurrentOperationTimestamp = 1;
	bool CheckInOperation(int32 Timestamp) const { return CurrentOperationTimestamp == Timestamp; }

	void CompleteMeshEditChange(const FText& TransactionLabel, TUniquePtr<FToolCommandChange> EditChange, const FGroupTopologySelection& OutputSelection);

	void AfterTopologyEdit();
	int32 ModifiedTopologyCounter = 0;
	bool bWasTopologyEdited = false;

	void ApplyDeleteAction();


	friend class FEditGroomCardsTopologyPreEditChange;
	friend class FEditGroomCardsTopologyPostEditChange;
	friend class FBeginGroomCardsEditChange;
	friend class FEndGroomCardsEditChange;

public:
	void ApplyGroomCardEdit(const FGroomCardEdit& Edit, bool bIsRevert);
};



class HAIRMODELINGTOOLSET_API FEditGroomCardsTopologyPreEditChange : public FToolCommandChange
{
public:
	virtual void Apply(UObject* Object) override;
	virtual void Revert(UObject* Object) override;
	virtual FString ToString() const override;
};

class HAIRMODELINGTOOLSET_API FEditGroomCardsTopologyPostEditChange : public FToolCommandChange
{
public:
	virtual void Apply(UObject* Object) override;
	virtual void Revert(UObject* Object) override;
	virtual FString ToString() const override;
};


class HAIRMODELINGTOOLSET_API FBeginGroomCardsEditChange : public FToolCommandChange
{
public:
	int32 CardGroupID = -1;
	FBeginGroomCardsEditChange(int32 GroupID = 0) { CardGroupID = GroupID; }

	virtual void Apply(UObject* Object) override;
	virtual void Revert(UObject* Object) override;
	virtual FString ToString() const override;
};


class HAIRMODELINGTOOLSET_API FEndGroomCardsEditChange : public FToolCommandChange
{
public:
	TPimplPtr<FGroomCardEdit> CardEdit;

	virtual void Apply(UObject* Object) override;
	virtual void Revert(UObject* Object) override;
	virtual FString ToString() const override;
};