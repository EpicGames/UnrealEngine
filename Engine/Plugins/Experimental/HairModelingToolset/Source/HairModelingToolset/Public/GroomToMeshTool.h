// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SingleSelectionTool.h"
#include "InteractiveToolBuilder.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "PreviewMesh.h"
#include "Drawing/PreviewGeometryActor.h"

#include "GroomToMeshTool.generated.h"

class AGroomActor;
class AStaticMeshActor;
PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);

/**
 *
 */
UCLASS()
class HAIRMODELINGTOOLSET_API UGroomToMeshToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()
public:
	virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};




UENUM()
enum class EGroomToMeshUVMode
{
	PlanarSplitting = 1,
	MinimalConformal = 2,
	PlanarSplitConformal = 3
};



UCLASS()
class HAIRMODELINGTOOLSET_API UGroomToMeshToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** The size of the geometry bounding box major axis measured in voxels */
	UPROPERTY(EditAnywhere, Category = Meshing, meta = (UIMin = "8", UIMax = "512", ClampMin = "8", ClampMax = "1024"))
	int32 VoxelCount = 64;

	UPROPERTY(EditAnywhere, Category = Meshing, meta = (UIMin = "0.5", UIMax = "16.0", ClampMin = "0.1", ClampMax = "128.0"))
	float BlendPower = 1.0;

	UPROPERTY(EditAnywhere, Category = Meshing, meta = (UIMin = "0.1", UIMax = "200.0", ClampMin = "0.1", ClampMax = "2000.0"))
	float RadiusScale = 100.0;


	UPROPERTY(EditAnywhere, Category = Morphology)
	bool bApplyMorphology = true;

	UPROPERTY(EditAnywhere, Category = Morphology, meta = (UIMin = "0.0", UIMax = "50.0", ClampMin = "0.0", ClampMax = "128.0"))
	float ClosingDist = 2.0;

	UPROPERTY(EditAnywhere, Category = Morphology, meta = (UIMin = "0.0", UIMax = "50.0", ClampMin = "0.0", ClampMax = "128.0"))
	float OpeningDist = 0.25;



	UPROPERTY(EditAnywhere, Category = Clipping)
	bool bClipToHead = true;

	// todo: this probably also needs to support skeletal mesh
	UPROPERTY(EditAnywhere, Category = Clipping)
	TSoftObjectPtr<AStaticMeshActor> ClipMeshActor;


	UPROPERTY(EditAnywhere, Category = Smoothing)
	bool bSmooth = true;

	UPROPERTY(EditAnywhere, Category = Smoothing, meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Smoothness = 0.15f;

	UPROPERTY(EditAnywhere, Category = Smoothing, meta = (UIMin = "-1.0", UIMax = "1.0", ClampMin = "-2.0", ClampMax = "2.0"))
	float VolumeCorrection = -0.25f;


	UPROPERTY(EditAnywhere, Category = Simplification)
	bool bSimplify = false;

	/** Target triangle count */
	UPROPERTY(EditAnywhere, Category = Simplification, meta = (UIMin = "4", UIMax = "5000", ClampMin = "1", ClampMax = "9999999999", EditCondition = "bSimplify == true"))
	int VertexCount = 500;


	UPROPERTY(EditAnywhere, Category = UVGeneration)
	EGroomToMeshUVMode UVMode = EGroomToMeshUVMode::MinimalConformal;


	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowSideBySide = true;

	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowGuides = false;

	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowUVs = false;

};



/**
 *
 */
UCLASS()
class HAIRMODELINGTOOLSET_API UGroomToMeshTool : public UInteractiveTool
{
	GENERATED_BODY()

public:
	UGroomToMeshTool();

	virtual void SetWorld(UWorld* World) { this->TargetWorld = World; }
	virtual void SetSelection(AGroomActor* Groom);

	virtual void Setup() override;
	virtual void Shutdown(EToolShutdownType ShutdownType) override;

	virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override;
	virtual bool CanAccept() const override;

protected:
	UPROPERTY()
	TObjectPtr<UGroomToMeshToolProperties> Settings;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh;

	UPROPERTY()
	TSoftObjectPtr<AGroomActor> TargetGroom;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> PreviewGeom;

protected:
	UPROPERTY()
	TObjectPtr<UMaterialInterface> MeshMaterial;

	UPROPERTY()
	TObjectPtr<UMaterialInterface> UVMaterial;

protected:
	UWorld* TargetWorld = nullptr;

	FDynamicMesh3 CurrentMesh;

	void RecalculateMesh();

	void UpdateLineSet();

	bool bResultValid = false;
	bool bVisualizationChanged = false;

	struct FVoxelizeSettings
	{
		int32 VoxelCount;
		float BlendPower;
		float RadiusScale;
		
		bool operator==(const FVoxelizeSettings& Other) const
		{
			return VoxelCount == Other.VoxelCount && BlendPower == Other.BlendPower && RadiusScale == Other.RadiusScale;
		}
	};
	FVoxelizeSettings CachedVoxelizeSettings;
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> CurrentVoxelizeResult;
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> UpdateVoxelization();



	struct FMorphologySettings
	{
		TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh;
		int32 VoxelCount;
		float CloseDist;
		float OpenDist;

		bool operator==(const FMorphologySettings& Other) const
		{
			return InputMesh.Get() == Other.InputMesh.Get() && VoxelCount == Other.VoxelCount && OpenDist == Other.OpenDist && CloseDist == Other.CloseDist;
		}
	};
	FMorphologySettings CachedMorphologySettings;
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> CachedMorphologyResult;
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> UpdateMorphology(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh);


	struct FClipMeshSettings
	{
		TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh;
		AActor* ClipSource;

		bool operator==(const FClipMeshSettings& Other) const
		{
			return InputMesh.Get() == Other.InputMesh.Get() && ClipSource == Other.ClipSource;
		}
	};
	FClipMeshSettings CachedClipMeshSettings;
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> CachedClipMeshResult;
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> UpdateClipMesh(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh);


	struct FSmoothingSettings
	{
		TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh;
		float Smoothness;
		float VolumeCorrection;

		bool operator==(const FSmoothingSettings& Other) const
		{
			return InputMesh.Get() == Other.InputMesh.Get() && Smoothness == Other.Smoothness && VolumeCorrection == Other.VolumeCorrection;
		}
	};
	FSmoothingSettings CachedSmoothSettings;
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> CachedSmoothResult;
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> UpdateSmoothing(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh);


	struct FSimplifySettings
	{
		TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh;
		int32 TargetCount;

		bool operator==(const FSimplifySettings& Other) const
		{
			return TargetCount == Other.TargetCount && InputMesh.Get() == Other.InputMesh.Get();
		}
	};
	FSimplifySettings CachedSimplifySettings;
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> CachedSimplifyResult;
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> UpdateSimplification(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh);


	struct FPostprocessSettings
	{
		TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh;
		EGroomToMeshUVMode UVGenMode;

		bool operator==(const FPostprocessSettings& Other) const
		{
			return InputMesh.Get() == Other.InputMesh.Get() && UVGenMode == Other.UVGenMode;
		}
	};
	FPostprocessSettings CachedPostprocessSettings;
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> CachedPostprocessResult;
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> UpdatePostprocessing(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh);


	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> UpdateUVs(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh, EGroomToMeshUVMode UVMode);

	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> UpdateUVs_ExpMapPlaneSplits(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh, bool bRecalcAsConformal);
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> UpdateUVs_MinimalConformal(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh);

	void UpdatePreview(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> ResultMesh);
};
