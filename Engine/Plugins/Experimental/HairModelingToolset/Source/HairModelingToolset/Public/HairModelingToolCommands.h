// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"

/**
 * TInteractiveToolCommands implementation for this module that provides standard Editor hotkey support
 */
class HAIRMODELINGTOOLSET_API FHairModelingToolCommands : public TCommands<FHairModelingToolCommands>
{
public:
	FHairModelingToolCommands();

	TSharedPtr<FUICommandInfo> HairToolsTabButton;

	TSharedPtr<FUICommandInfo> BeginGroomToMeshTool;
	TSharedPtr<FUICommandInfo> BeginGenerateLODMeshesTool;
	TSharedPtr<FUICommandInfo> BeginGroomCardsEditorTool;

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;
};
